// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IOSAudio : ModuleRules
{
	public IOSAudio(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		BinariesSubFolder = "IOS";

		PrivateIncludePathModuleNames.Add("TargetPlatform");

		PublicIncludePaths.AddRange(new string[]
		{
		});
		
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
            "AudioMixer"
		});

		PublicFrameworks.AddRange(new string[]
		{
			"AudioToolbox",
			"CoreAudio",
			"AVFoundation"
		});
	}
}
