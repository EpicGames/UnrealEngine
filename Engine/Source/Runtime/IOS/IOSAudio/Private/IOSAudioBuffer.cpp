// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IOSAudioBuffer.cpp: Unreal IOSAudio buffer interface object.
 =============================================================================*/

/*------------------------------------------------------------------------------------
	Includes
 ------------------------------------------------------------------------------------*/

#include "IOSAudioDevice.h"
#include "AudioEffect.h"
#include "Interfaces/IAudioFormat.h"
#include "Sound/SoundWave.h"
#include "AudioDeviceManager.h"
#include "Engine/Engine.h"
#include "AudioDecompress.h"

/*------------------------------------------------------------------------------------
	FIOSAudioSoundBuffer
 ------------------------------------------------------------------------------------*/

FIOSAudioSoundBuffer::FIOSAudioSoundBuffer(FIOSAudioDevice* InAudioDevice, USoundWave* InWave, bool InStreaming, bool InProcedural):
	FSoundBuffer(InAudioDevice),
	SampleData(nullptr),
	BufferSize(0),
	DecompressionState(nullptr),
	bStreaming(InStreaming),
	bIsProcedural(InProcedural)
{
	if (!bIsProcedural)
	{
		if (!ReadCompressedInfo(InWave))
		{
			return;
		}

		SoundFormat = SoundFormat_LPCM; 
	}
    else
    {
        SoundFormat = SoundFormat_LPCM;
    }
	
	
	SampleRate = InWave->GetSampleRateForCurrentPlatform();
	NumChannels = InWave->NumChannels;
	BufferSize = AudioCallbackFrameSize * sizeof(uint16) * NumChannels;
	SampleData = static_cast<int16*>(FMemory::Malloc(BufferSize));
	check(SampleData != nullptr);
    FMemory::Memzero(SampleData, BufferSize);
    
	
	FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();
	check(AudioDeviceManager != nullptr);

	// There is no need to track this resource with the AudioDeviceManager since there is a one to one mapping between FIOSAudioSoundBuffer objects and FIOSAudioSoundSource objects
	//	and this object will be deleted when the corresponding FIOSAudioSoundSource no longer needs it
}

FIOSAudioSoundBuffer::~FIOSAudioSoundBuffer(void)
{
	if (bAllocationInPermanentPool)
	{
		UE_LOGF(LogIOSAudio, Fatal, "Can't free resource '%ls' as it was allocated in permanent pool.", *ResourceName);
	}
	
	if(SampleData != nullptr)
	{
		FMemory::Free(SampleData);
		SampleData = nullptr;
	}
	
	if(DecompressionState != nullptr)
	{
		delete DecompressionState;
		DecompressionState = nullptr;
	}
}

int32 FIOSAudioSoundBuffer::GetSize(void)
{
	int32 TotalSize = 0;
	
	switch (SoundFormat)
	{
		case SoundFormat_LPCM:
		case SoundFormat_ADPCM:
			TotalSize = BufferSize;
			break;
	}
	
	return TotalSize;
}

int32 FIOSAudioSoundBuffer::GetCurrentChunkIndex() const
{
	if(DecompressionState == nullptr)
	{
		return -1;
	}
	
	return DecompressionState->GetCurrentChunkIndex();
}

int32 FIOSAudioSoundBuffer::GetCurrentChunkOffset() const
{
	if(DecompressionState == nullptr)
	{
		return -1;
	}
	
	return DecompressionState->GetCurrentChunkOffset();
}

bool FIOSAudioSoundBuffer::ReadCompressedInfo(USoundWave* InWave)
{
	check(DecompressionState != nullptr);

	FSoundQualityInfo QualityInfo = { 0 };
	if(bStreaming)
	{
		return DecompressionState->StreamCompressedInfo(InWave, &QualityInfo);
	}

	InWave->InitAudioResource(FName("ADPCM"));
	if (InWave->GetResourceSize() <= 0)
	{
		InWave->RemoveAudioResource();
		return false;
	}

	return DecompressionState->ReadCompressedInfo(InWave->GetResourceData(), InWave->GetResourceSize(), &QualityInfo);
}

bool FIOSAudioSoundBuffer::ReadCompressedData( uint8* Destination, int32 NumFramesToDecode, bool bLooping )
{
	int32 NumFramesDecoded = NumFramesToDecode;
	if(bStreaming)
	{
		return DecompressionState->StreamCompressedData(Destination, bLooping, RenderCallbackBufferSize, NumFramesDecoded);
	}
	else
	{
		return DecompressionState->ReadCompressedData(Destination, bLooping, RenderCallbackBufferSize);
	}
}

FIOSAudioSoundBuffer* FIOSAudioSoundBuffer::Init(FIOSAudioDevice* IOSAudioDevice, USoundWave* InWave)
{
	// Can't create a buffer without any source data
	if (InWave == NULL || InWave->NumChannels == 0)
	{
		return NULL;
	}

	FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();

	FIOSAudioSoundBuffer *Buffer = NULL;
	bool	bIsStreaming = false;
	
	switch (static_cast<EDecompressionType>(InWave->DecompressionType))
	{
		case DTYPE_Setup:
			// Has circumvented pre-cache mechanism - pre-cache now
			IOSAudioDevice->Precache(InWave, true, false);
			
			// Recall this function with new decompression type
			return Init(IOSAudioDevice, InWave);
			
		case DTYPE_Streaming:
			bIsStreaming = true;
			// fall through to next case
						
		case DTYPE_RealTime:
			// Always create a new FIOSAudioSoundBuffer since positional information about the sound is kept track of in this object
			Buffer = new FIOSAudioSoundBuffer(IOSAudioDevice, InWave, bIsStreaming, false);
			break;

		case DTYPE_Procedural:
			Buffer = new FIOSAudioSoundBuffer(IOSAudioDevice, InWave, bIsStreaming, true);
			break;

		case DTYPE_Invalid:
		default:
			// Invalid will be set if the wave cannot be played
			UE_LOGF( LogIOSAudio, Warning, "Init Buffer on unsupported sound type name = %ls type = %d", *InWave->GetName(), int32(InWave->DecompressionType));
			break;
	}

	return Buffer;
}

bool FIOSAudioSoundBuffer::ReleaseCurrentChunk()
{
	if (DecompressionState && bStreaming)
	{
		return DecompressionState->ReleaseStreamChunk(true);
	}
	else
	{
		return true;
	}
}
