// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 	IOSAudioDevice.cpp: Unreal IOSAudio audio interface object.
 =============================================================================*/

/*------------------------------------------------------------------------------------
	Includes
 ------------------------------------------------------------------------------------*/

#include "IOSAudioDevice.h"
#include "AudioEffect.h"
#include "AudioPluginUtilities.h"
#include "AudioDecompress.h"

DEFINE_LOG_CATEGORY(LogIOSAudio);

class FIOSAudioDeviceModule : public IAudioDeviceModule
{
public:

	/** Creates a new instance of the audio device implemented by the module. */
	virtual FAudioDevice* CreateAudioDevice() override
	{
		return new FIOSAudioDevice;
	}
};

IMPLEMENT_MODULE(FIOSAudioDeviceModule, IOSAudio);

/*------------------------------------------------------------------------------------
	FIOSAudioDevice
 ------------------------------------------------------------------------------------*/

static int32 SuspendCounter = 0;

void FIOSAudioDevice::ResumeContext()
{
	if (SuspendCounter > 0)
	{
		FPlatformAtomics::InterlockedDecrement(&SuspendCounter);
        
		if (AudioUnitGraph != NULL)
		{
			AUGraphStart(AudioUnitGraph);
		}
		
		if (OutputUnit != NULL)
		{
			AudioOutputUnitStart(OutputUnit);
		}
		
		UE_LOGF(LogIOSAudio, Display, "Resuming Audio");
	}
}

void FIOSAudioDevice::SuspendContext()
{
	if (SuspendCounter == 0)
	{
		FPlatformAtomics::InterlockedIncrement(&SuspendCounter);

		if (OutputUnit != NULL)
		{
			AudioOutputUnitStop(OutputUnit);
		}

		if (AudioUnitGraph != NULL)
		{
			AUGraphStop(AudioUnitGraph);
		}
		
		UE_LOGF(LogIOSAudio, Display, "Suspending Audio");
	}
}

void FIOSAudioDevice::IncrementSuspendCounter()
{
    if(SuspendCounter == 0)
    {
        FPlatformAtomics::InterlockedIncrement(&SuspendCounter);
    }
}

void FIOSAudioDevice::DecrementSuspendCounter()
{
    if (SuspendCounter > 0)
    {
        FPlatformAtomics::InterlockedDecrement(&SuspendCounter);
    }
}

void FIOSAudioDevice::UpdateDeviceDeltaTime()
{
	DeviceDeltaTime = GetGameDeltaTime();
}

FIOSAudioDevice::FIOSAudioDevice() :
	FAudioDevice(),
	AudioUnitGraph(NULL),
	OutputNode(0),
	OutputUnit(NULL),
	MixerNode(0),
	MixerUnit(NULL),
	NextBusNumber(0)
{
}

bool FIOSAudioDevice::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("DumpAUGraph")) && AudioUnitGraph)
	{
		CAShow(AudioUnitGraph);
		return true;
	}
	
	return FAudioDevice::Exec(InWorld, Cmd, Ar);
}

FAudioPlatformSettings FIOSAudioDevice::GetPlatformSettings() const
{
	return FAudioPlatformSettings::GetPlatformSettings(FPlatformProperties::GetRuntimeSettingsClassName());
}

bool FIOSAudioDevice::InitializeHardware()
{
    UE_LOGF(LogIOSAudio, Warning, "Initializing legacy audio backend for iOS");
	SIZE_T SampleSize = sizeof(AudioSampleType);
	double GraphSampleRate = 44100.0;

	if (!SetHardwareSampleRate(GraphSampleRate) || !SetAudioSessionActive(true))
	{
		HandleError(TEXT("Failed to establish the audio session!"));
		return false;
	}

	// Retrieve the actual hardware sample rate
	GetHardwareSampleRate(GraphSampleRate);

	// Linear PCM stream format
	MixerFormat.mFormatID         = kAudioFormatLinearPCM;
	MixerFormat.mFormatFlags	  = kAudioFormatFlagIsSignedInteger | kAudioFormatFlagsNativeEndian | kAudioFormatFlagIsPacked;
	MixerFormat.mBytesPerPacket   = SampleSize;
	MixerFormat.mFramesPerPacket  = 1;
	MixerFormat.mBytesPerFrame    = SampleSize;
	MixerFormat.mChannelsPerFrame = 1;
	MixerFormat.mBitsPerChannel   = 8 * SampleSize;
	MixerFormat.mSampleRate       = GraphSampleRate;

	OSStatus Status = NewAUGraph(&AudioUnitGraph);
	if (Status != noErr)
	{
		HandleError(TEXT("Failed to create audio unit graph!"));
		return false;
	}

	AudioComponentDescription UnitDescription;

	// Setup audio output unit
	UnitDescription.componentType         = kAudioUnitType_Output;
	UnitDescription.componentSubType      = kAudioUnitSubType_RemoteIO;
	UnitDescription.componentManufacturer = kAudioUnitManufacturer_Apple;
	UnitDescription.componentFlags        = 0;
	UnitDescription.componentFlagsMask    = 0;
	Status = AUGraphAddNode(AudioUnitGraph, &UnitDescription, &OutputNode);
	if (Status != noErr)
	{
		HandleError(TEXT("Failed to initialize audio output node!"), true);
		return false;
	}

	// Setup audo mixer unit
	UnitDescription.componentType         = kAudioUnitType_Mixer;
    UnitDescription.componentSubType      = kAudioUnitSubType_SpatialMixer;
	UnitDescription.componentManufacturer = kAudioUnitManufacturer_Apple;
	UnitDescription.componentFlags        = 0;
	UnitDescription.componentFlagsMask    = 0;
	Status = AUGraphAddNode(AudioUnitGraph, &UnitDescription, &MixerNode);
	if (Status != noErr)
	{
		HandleError(TEXT("Failed to initialize audio mixer node!"), true);
		return false;
	}
	
	Status = AUGraphOpen(AudioUnitGraph);
	if (Status != noErr)
	{
		HandleError(TEXT("Failed to open audio unit graph"), true);
		return false;
	}
	
	Status = AUGraphNodeInfo(AudioUnitGraph, OutputNode, NULL, &OutputUnit);
	if (Status != noErr)
	{
		HandleError(TEXT("Failed to retrieve output unit reference!"), true);
		OutputUnit = NULL;
		return false;
	}
	
	Status = AUGraphNodeInfo(AudioUnitGraph, MixerNode, NULL, &MixerUnit);
	if (Status != noErr)
	{
		HandleError(TEXT("Failed to retrieve mixer unit reference!"), true);
		MixerUnit = NULL;
		return false;
	}

	uint32 BusCount = GetMaxSources() * CHANNELS_PER_BUS;
	Status = AudioUnitSetProperty(MixerUnit,
	                              kAudioUnitProperty_ElementCount,
	                              kAudioUnitScope_Input,
	                              0,
	                              &BusCount,
	                              sizeof(BusCount));
	if (Status != noErr)
	{
		HandleError(TEXT("Failed to set kAudioUnitProperty_ElementCount for audio mixer unit!"), true);
		return false;
	}
	
	// Initialize sound source early on, allowing for render callback hookups
	InitSoundSources();

	// Setup the mixer unit sample rate
	Status = AudioUnitSetProperty(MixerUnit,
	                              kAudioUnitProperty_SampleRate,
	                              kAudioUnitScope_Output,
	                              0,
	                              &GraphSampleRate,
	                              sizeof(GraphSampleRate));
	if (Status != noErr)
	{
		HandleError(TEXT("Failed to set kAudioUnitProperty_SampleRate for audio mixer unit!"), true);
		return false;
	}
	
	// Connect mixer node output to output node input
	Status = AUGraphConnectNodeInput(AudioUnitGraph, MixerNode, 0, OutputNode, 0);
	if (Status != noErr)
	{
		HandleError(TEXT("Failed to connect mixer node to output node!"), true);
		return false;
	}

	// Initialize and start the audio unit graph
	Status = AUGraphInitialize(AudioUnitGraph);
	if (Status == noErr && SuspendCounter == 0)
	{
		Status = AUGraphStart(AudioUnitGraph);
	}

	if (Status != noErr)
	{
		HandleError(TEXT("Failed to start audio graph!"), true);
		return false;
	}

	return true;
}

void FIOSAudioDevice::TeardownHardware()
{
	if (AudioUnitGraph)
	{
		AUGraphStop(AudioUnitGraph);
		DisposeAUGraph(AudioUnitGraph);

		AudioUnitGraph = NULL;
		OutputNode = -1;
		OutputUnit = NULL;
		MixerNode = -1;
		MixerUnit = NULL;
	}
}

void FIOSAudioDevice::UpdateHardware()
{
	const FListener& Listener = GetListeners()[0];
	PlayerLocation	= Listener.Transform.GetLocation();
	PlayerFacing	= Listener.GetFront();
	PlayerUp		= Listener.GetUp();
	PlayerRight		= Listener.GetRight();
}

void FIOSAudioDevice::HandleError(const TCHAR* InLogOutput, bool bTeardown)
{
	UE_LOGF(LogIOSAudio, Log, "%ls", InLogOutput);
	if (bTeardown)
	{
		Teardown();
	}
}

FAudioEffectsManager* FIOSAudioDevice::CreateEffectsManager()
{
	// Create the basic no-op effects manager
	return FAudioDevice::CreateEffectsManager();
}

FSoundSource* FIOSAudioDevice::CreateSoundSource()
{
	return new FIOSAudioSoundSource(this, NextBusNumber++);
}

