// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 	IOSAudioSession.mm: Unreal IOSAudio audio session functionality.
 =============================================================================*/

/*------------------------------------------------------------------------------------
	Includes
 ------------------------------------------------------------------------------------*/

#include "IOSAudioDevice.h"
#include "AudioEffect.h"

/*------------------------------------------------------------------------------------
	FIOSAudioDevice
 ------------------------------------------------------------------------------------*/

void FIOSAudioDevice::GetHardwareSampleRate(double& OutSampleRate)
{
	AVAudioSession* AudioSession = [AVAudioSession sharedInstance];
	OutSampleRate = [AudioSession preferredSampleRate];
}

bool FIOSAudioDevice::SetHardwareSampleRate(const double& InSampleRate)
{
	AVAudioSession* AudioSession = [AVAudioSession sharedInstance];
	return [AudioSession setPreferredSampleRate:InSampleRate error:nil] == YES;
}

bool FIOSAudioDevice::SetAudioSessionActive(bool bActive)
{
	AVAudioSession* AudioSession = [AVAudioSession sharedInstance];
	return [AudioSession setActive:bActive error:nil] == YES;
}

bool FIOSAudioDevice::IsExernalBackgroundSoundActive()
{
	return [[AVAudioSession sharedInstance] isOtherAudioPlaying];
}