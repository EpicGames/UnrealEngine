// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IOSAudioSource.cpp: Unreal IOSAudio source interface object.
 =============================================================================*/

/*------------------------------------------------------------------------------------
	Includes
 ------------------------------------------------------------------------------------*/

#include "IOSAudioDevice.h"
#include "Interfaces/IAudioFormat.h"
#include "ContentStreaming.h"
#include "Interfaces/IAudioFormat.h"
#include "AudioDecompress.h"
#include "Audio.h"

const uint32	Callback_Free = 0;
const uint32	Callback_Locked = 1;

namespace
{
	inline bool LockCallback(int32* InCallbackLock)
	{
		check(InCallbackLock != NULL);
		return FPlatformAtomics::InterlockedCompareExchange(InCallbackLock, Callback_Locked, Callback_Free) == Callback_Free;
	}

	inline void UnlockCallback(int32* InCallbackLock)
	{
		check(InCallbackLock != NULL);
		int32 Result = FPlatformAtomics::InterlockedCompareExchange(InCallbackLock, Callback_Free, Callback_Locked);

		check(Result == Callback_Locked);
	}

} // end namespace

/*------------------------------------------------------------------------------------
	FIOSAudioSoundSource
 ------------------------------------------------------------------------------------*/

FIOSAudioSoundSource::FIOSAudioSoundSource(FIOSAudioDevice* InAudioDevice, uint32 InBusNumber) :
	FSoundSource(InAudioDevice),
	SampleRate(0),
    SourceLPFFrequency(MAX_FILTER_FREQUENCY),
	IOSAudioDevice(InAudioDevice),
	IOSBuffer(NULL),
	BusNumber(InBusNumber)
{
	check(IOSAudioDevice);
	
	WaveInstance = NULL;
	
	// Start in a disabled state
	DetachFromAUGraph();
	SampleRate = static_cast<int32>(IOSAudioDevice->MixerFormat.mSampleRate);

	AURenderCallbackStruct Input;	
	Input.inputProc = &IOSAudioRenderCallback;
	Input.inputProcRefCon = this;
	
	CallbackLock = Callback_Free;
	
	OSStatus Status = noErr;
	for (int32 Channel = 0; Channel < CHANNELS_PER_BUS; Channel++)
	{
		Status = AudioUnitSetProperty(IOSAudioDevice->GetMixerUnit(),
		                              kAudioUnitProperty_StreamFormat,
		                              kAudioUnitScope_Input,
		                              GetAudioUnitElement(Channel),
		                              &IOSAudioDevice->MixerFormat,
		                              sizeof(AudioStreamBasicDescription));
		UE_CLOGF(Status != noErr, LogIOSAudio, Error, "Failed to set kAudioUnitProperty_StreamFormat for audio mixer unit: BusNumber=%d, Channel=%d", BusNumber, Channel);

		Status = AudioUnitSetParameter(IOSAudioDevice->GetMixerUnit(),
		                               k3DMixerParam_Distance,
		                               kAudioUnitScope_Input,
		                               GetAudioUnitElement(Channel),
		                               1.0f,
		                               0);
		UE_CLOGF(Status != noErr, LogIOSAudio, Error, "Failed to set k3DMixerParam_Distance for audio mixer unit: BusNumber=%d, Channel=%d", BusNumber, Channel);

		Status = AUGraphSetNodeInputCallback(IOSAudioDevice->GetAudioUnitGraph(),
		                                     IOSAudioDevice->GetMixerNode(), 
		                                     GetAudioUnitElement(Channel),
		                                     &Input);
		UE_CLOGF(Status != noErr, LogIOSAudio, Error, "Failed to set input callback for audio mixer node: BusNumber=%d, Channel=%d", BusNumber, Channel);
	}
}

FIOSAudioSoundSource::~FIOSAudioSoundSource(void)
{
	// Ensure we are stopped and detached from the audio graph so that playback has been stopped to prevent any chance this object being deleted during the audio render callback function
	Stop();
	
	WaveInstance = NULL;
	if(IOSBuffer != NULL)
	{
		delete IOSBuffer;
		IOSBuffer = NULL;
	}
}

void FIOSAudioSoundSource::CleanupAudioBuffer()
{
	// Always enure we have a unique FIOSAudioSoundBuffer and that we delete the old one if it exists
	if (IOSBuffer != NULL)
	{
		delete IOSBuffer;
		IOSBuffer = NULL;
	}
}


bool FIOSAudioSoundSource::Init(FWaveInstance* InWaveInstance)
{
	// Wait for the render callback to finish and then prevent it from being entered again in case this object is deleted after being stopped
	while (!LockCallback(&CallbackLock))
	{
		UE_LOGF(LogIOSAudio, Log, "Waiting for source to unlock");
		
		// Allow time for other threads to run
		FPlatformProcess::Sleep(0.0f);
	}
	
	FSoundSource::InitCommon();

	if (InWaveInstance->OutputTarget == EAudioOutputTarget::Controller)
	{
		UnlockCallback(&CallbackLock);
		return false;
	}
	
	CleanupAudioBuffer();
	
	IOSBuffer = FIOSAudioSoundBuffer::Init(IOSAudioDevice, InWaveInstance->WaveData);

	if (IOSBuffer == NULL || IOSBuffer->NumChannels <= 0 || (IOSBuffer->SoundFormat != SoundFormat_LPCM && IOSBuffer->SoundFormat != SoundFormat_ADPCM))
	{
		UnlockCallback(&CallbackLock);
		return false;
	}

	SCOPE_CYCLE_COUNTER(STAT_AudioSourceInitTime);
	
	WaveInstance = InWaveInstance;
	
	bAllChannelsFinished = false;
	
	AudioStreamBasicDescription StreamFormat;

	SampleRate = IOSBuffer->SampleRate;

	StreamFormat = IOSAudioDevice->MixerFormat;
	StreamFormat.mSampleRate = static_cast<Float64>(SampleRate);

	OSStatus Status = noErr;
	for (int32 Channel = 0; Channel < IOSBuffer->NumChannels; ++Channel)
	{
		Status = AudioUnitSetProperty(IOSAudioDevice->GetMixerUnit(),
									  kAudioUnitProperty_StreamFormat,
									  kAudioUnitScope_Input,
									  GetAudioUnitElement(Channel),
									  &StreamFormat,
									  sizeof(AudioStreamBasicDescription));
		UE_CLOGF(Status != noErr, LogIOSAudio, Error, "Failed to set kAudioUnitProperty_StreamFormat for audio mixer unit: BusNumber=%d, Channel=%d", BusNumber, Channel);

		AudioUnitParameterValue Pan = 0.0f;
		if(IOSBuffer->NumChannels == 2)
		{
			const AudioUnitParameterValue AzimuthRangeScale = 90.f;
			Pan = (-1.0f + (Channel * 2.0f)) * AzimuthRangeScale;
		}
		else if (!WaveInstance->GetUseSpatialization())
		{
			Pan = 0.0f;
		}

		Status = AudioUnitSetParameter(IOSAudioDevice->GetMixerUnit(),
		                               k3DMixerParam_Azimuth,
		                               kAudioUnitScope_Input,
		                               GetAudioUnitElement(Channel),
		                               Pan,
		                               0);
		UE_CLOGF(Status != noErr, LogIOSAudio, Error, "Failed to set k3DMixerParam_Azimuth for audio mixer unit: BusNumber=%d, Channel=%d", BusNumber, Channel);
	}

	// Seek into the file if we've been given a non-zero start time.
	if (WaveInstance->IsSeekable() && WaveInstance->StartTime > 0.0f)
	{
		IOSBuffer->DecompressionState->SeekToTime(WaveInstance->StartTime);
	}

	// Start in a disabled state
	DetachFromAUGraph();
	Update();
	
	UnlockCallback(&CallbackLock);

    LowpassFilterBank.SetNum(CHANNELS_PER_BUS);
    LPFParamBank.SetNum(CHANNELS_PER_BUS);
    
	return true;
}

void FIOSAudioSoundSource::Update(void)
{
	SCOPE_CYCLE_COUNTER(STAT_AudioUpdateSources);

	if (!WaveInstance || Paused)
	{
		return;
	}

	FSoundSource::UpdateCommon();

	AudioUnitParameterValue Volume = 0.0f;

	if (!AudioDevice->IsAudioDeviceMuted())
	{
		Volume = WaveInstance->GetActualVolume();
	}

    SetFilterFrequency();
    
    SourceLPFFrequency = LPFFrequency / (((float) SampleRate) * 0.5f);
    
	// Factor in the xaudio2 attenuation that happens to stereo assets.
	if (WaveInstance->WaveData->NumChannels == 2 && WaveInstance->GetUseSpatialization())
	{
		Volume *= 0.5f;
	}

	// Apply global multiplier to disable sound when not the foreground app
	Volume *= AudioDevice->GetPlatformAudioHeadroom();
	Volume = FMath::Clamp(Volume, 0.0f, 1.0f);

	// Convert to dB
	const AudioUnitParameterValue Gain = FMath::Clamp<float>(20.0f * log10(Volume), -100, 0.0f);
	const AudioUnitParameterValue PitchParam = Pitch;

	OSStatus Status = noErr;

	// We only adjust panning on playback for mono sounds that want spatialization
	if (IOSBuffer->NumChannels == 1 && WaveInstance->GetUseSpatialization())
	{
		// Compute the directional offset
		FVector Offset = GetSpatializationParams().EmitterPosition;
		const AudioUnitParameterValue AzimuthRangeScale = 90.0f;
        const AudioUnitParameterValue Pan = Offset.Y * AzimuthRangeScale;

		Status = AudioUnitSetParameter(IOSAudioDevice->GetMixerUnit(),
		                               k3DMixerParam_Azimuth,
		                               kAudioUnitScope_Input,
		                               GetAudioUnitElement(0),
		                               Pan,
		                               0);
		UE_CLOGF(Status != noErr, LogIOSAudio, Error, "Failed to set k3DMixerParam_Azimuth for audio mixer unit: BusNumber=%d, Channel=%d", BusNumber, 0);
	}

	for (int32 Channel = 0; Channel < IOSBuffer->NumChannels; Channel++)
	{
		Status = AudioUnitSetParameter(IOSAudioDevice->GetMixerUnit(),
		                               k3DMixerParam_Gain,
		                               kAudioUnitScope_Input,
		                               GetAudioUnitElement(Channel),
		                               Gain,
		                               0);
		UE_CLOGF(Status != noErr, LogIOSAudio, Error, "Failed to set k3DMixerParam_Gain for audio mixer unit: BusNumber=%d, Channel=%d", BusNumber, Channel);

		Status = AudioUnitSetParameter(IOSAudioDevice->GetMixerUnit(),
		                               k3DMixerParam_PlaybackRate,
		                               kAudioUnitScope_Input,
		                               GetAudioUnitElement(Channel),
									   PitchParam,
		                               0);
		UE_CLOGF(Status != noErr, LogIOSAudio, Error, "Failed to set k3DMixerParam_PlaybackRate for audio mixer unit: BusNumber=%d, Channel=%d", BusNumber, Channel);
	}
}

void FIOSAudioSoundSource::Play(void)
{
	if (WaveInstance && AttachToAUGraph())
	{
        
		Paused = false;
		Playing = true;

		// Updates the source which sets the pitch and volume
		Update();
        
        for (Audio::FOnePoleLPF& Filter : LowpassFilterBank)
        {
            Filter.Reset();
        }
        
        for (Audio::FParam& Param : LPFParamBank)
        {
            Param.Init();
            Param.SetValue(LPFFrequency);
            Param.Reset();
        }
        
	}
}

void FIOSAudioSoundSource::Stop(void)
{
	// Wait for the render callback to finish and then prevent it from being entered again in case this object is deleted after being stopped
	while (!LockCallback(&CallbackLock))
	{
		UE_LOGF(LogIOSAudio, Log, "Waiting for source to unlock");

		// Allow time for other threads to run
		FPlatformProcess::Sleep(0.0f);
	}

	if (WaveInstance)
	{
		Pause();

		Paused = false;
		Playing = false;
	}

	// Call parent class version regardless of if there's a wave instance
	FSoundSource::Stop();

	if (IOSBuffer != NULL)
	{
		// Release the current stream chunk:
		bool bChunkReleased = IOSBuffer->ReleaseCurrentChunk();
		check(bChunkReleased);
	}

	// It's now safe to unlock the callback
	UnlockCallback(&CallbackLock);
}

void FIOSAudioSoundSource::Pause(void)
{
	if (WaveInstance)
	{	
		if (Playing)
		{
			DetachFromAUGraph();
		}

		Paused = true;
	}
}

bool FIOSAudioSoundSource::IsFinished(void)
{
	// A paused source is not finished.
	if (Paused)
	{
		return false;
	}
	
	/* TODO::JTM - Jan 07, 2013 02:56PM - Properly handle wave instance notifications */
	if (WaveInstance && Playing)
	{
		if (WaveInstance->LoopingMode == LOOP_Never)
		{
			return bAllChannelsFinished;
		}
		
		if (WaveInstance->LoopingMode == LOOP_WithNotification)
		{
			// Notify the wave instance that the looping callback was hit
			if (bAllChannelsFinished)
			{
				WaveInstance->NotifyFinished();
			}
		}
		
		bAllChannelsFinished = false;

		return false;
	}

	return true;
}

AudioUnitElement FIOSAudioSoundSource::GetAudioUnitElement(int32 Channel)
{
	check(Channel < CHANNELS_PER_BUS);
	return BusNumber * CHANNELS_PER_BUS + static_cast<uint32>(Channel);
}

bool FIOSAudioSoundSource::AttachToAUGraph()
{
	OSStatus Status = noErr;

	// Set a callback for the specified node's specified input
	for (int32 Channel = 0; Channel < IOSBuffer->NumChannels; Channel++)
	{
		Status = AudioUnitSetParameter(IOSAudioDevice->GetMixerUnit(),
		                               k3DMixerParam_Enable, 
		                               kAudioUnitScope_Input,
		                               GetAudioUnitElement(Channel),
		                               1,
		                               0);
		UE_CLOGF(Status != noErr, LogIOSAudio, Error, "Failed to set k3DMixerParam_Enable for audio mixer unit: BusNumber=%d, Channel=%d", BusNumber, Channel);
	}

	return Status == noErr;
}

bool FIOSAudioSoundSource::DetachFromAUGraph()
{
	OSStatus Status = noErr;

	// Set a callback for the specified node's specified input
	for (int32 Channel = 0; Channel < CHANNELS_PER_BUS; Channel++)
	{
        Status = AudioUnitSetParameter(IOSAudioDevice->GetMixerUnit(),
                                       k3DMixerParam_Gain,
                                       kAudioUnitScope_Input,
                                       GetAudioUnitElement(Channel),
                                       -120.0,
                                       0);
        UE_CLOGF(Status != noErr, LogIOSAudio, Error, "Failed to set k3DMixerParam_Gain for audio mixer unit: BusNumber=%d, Channel=%d", BusNumber, Channel);
        
		Status = AudioUnitSetParameter(IOSAudioDevice->GetMixerUnit(),
		                               k3DMixerParam_Enable,
		                               kAudioUnitScope_Input,
		                               GetAudioUnitElement(Channel),
		                               0,
		                               0);
		UE_CLOGF(Status != noErr, LogIOSAudio, Error, "Failed to set k3DMixerParam_Enable for audio mixer unit: BusNumber=%d, Channel=%d", BusNumber, Channel);
	}

	return Status == noErr;
}

OSStatus FIOSAudioSoundSource::IOSAudioRenderCallback(void* RefCon, AudioUnitRenderActionFlags* ActionFlags,
                                                      const AudioTimeStamp* TimeStamp, UInt32 BusNumber,
                                                      UInt32 NumFrames, AudioBufferList* IOData)
{
	FIOSAudioSoundSource* Source = static_cast<FIOSAudioSoundSource*>(RefCon);
	UInt32 Channel = BusNumber % CHANNELS_PER_BUS;
    
	AudioSampleType* OutData = reinterpret_cast<AudioSampleType*>(IOData->mBuffers[0].mData);
    
	// Make sure we should be rendering
	if (!LockCallback(&Source->CallbackLock))
	{
		FMemory::Memzero(OutData, NumFrames * sizeof(AudioSampleType));
		return -1;
	}
	
	if (!Source->IOSBuffer || Channel > Source->IOSBuffer->NumChannels || !Source->IsPlaying() || Source->IsPaused() || (Source->WaveInstance->LoopingMode == LOOP_Never && Source->bAllChannelsFinished))
	{
		UnlockCallback(&Source->CallbackLock);
		FMemory::Memzero(OutData, NumFrames * sizeof(AudioSampleType));
		return -1;
	}

    
    
	if(Channel == 0)
	{
		// Grab the interleaved channel data
		if (Source->IOSBuffer->bIsProcedural)
		{
            Source->IOSBuffer->RenderCallbackBufferSize = NumFrames * sizeof(uint16) * Source->IOSBuffer->NumChannels;
            check(Source->IOSBuffer->RenderCallbackBufferSize <= Source->IOSBuffer->BufferSize);
            
            FMemory::Memzero(Source->IOSBuffer->SampleData, Source->IOSBuffer->RenderCallbackBufferSize);
            
			int32 DataSize = Source->WaveInstance->WaveData->GeneratePCMData(
				(uint8*) Source->IOSBuffer->SampleData,
				Source->IOSBuffer->RenderCallbackBufferSize / sizeof(int16));
			Source->bChannel0Finished = DataSize <= 0;
		}
		else
		{
			Source->IOSBuffer->RenderCallbackBufferSize = NumFrames * sizeof(uint16) * Source->IOSBuffer->NumChannels;

			// Since StreamCompressedData returns interlaced samples we need to decompress all frames(samples) for all channels here so we don't end up decompressing multiple times
			// Ensure we have enough memory to do this. If needed we could realloc here but that is bad practice inside the audio callback since it has a hard deadline
			check(Source->IOSBuffer->RenderCallbackBufferSize <= Source->IOSBuffer->BufferSize);

			Source->bChannel0Finished =
				Source->IOSBuffer->ReadCompressedData(
				(uint8*)Source->IOSBuffer->SampleData,
				MONO_PCM_BUFFER_SAMPLES,
				Source->WaveInstance->LoopingMode == LOOP_WithNotification || Source->WaveInstance->LoopingMode == LOOP_Forever);
		}
	}
    
    // If the channel count is higher than we've expected,
    // initialize a new LPF for this channel.
    if (Source->LPFParamBank.Num() <= Channel)
    {
        int32 ChannelIndex = Source->LPFParamBank.AddDefaulted(1);
        Source->LPFParamBank[ChannelIndex].Init();
        Source->LPFParamBank[ChannelIndex].SetValue(Source->SourceLPFFrequency);
        Source->LPFParamBank[ChannelIndex].Reset();
        
        ChannelIndex = Source->LowpassFilterBank.AddDefaulted(1);
        Source->LowpassFilterBank[ChannelIndex].Reset();
    }
    
    // Set up LPF filter for this channel:
    Audio::FParam& LPFParam = Source->LPFParamBank[Channel];
    Audio::FOnePoleLPF& LowpassFilter = Source->LowpassFilterBank[Channel];
    
    LPFParam.Reset();
    const bool bShouldUpdateCutoffFrequency = !FMath::IsNearlyEqual(Source->SourceLPFFrequency, LPFParam.GetValue());
    
    if(bShouldUpdateCutoffFrequency)
    {
        LPFParam.SetValue(Source->SourceLPFFrequency, NumFrames);
    }
    else
    {
        LowpassFilter.SetFrequency(Source->SourceLPFFrequency);
    }
    
	for(int32 sampleItr = 0; sampleItr < NumFrames; ++sampleItr)
	{
		AudioSampleType IntSample = *(Source->IOSBuffer->SampleData + sampleItr * Source->IOSBuffer->NumChannels + Channel);
        
        // Apply LPF:
        if(bShouldUpdateCutoffFrequency)
        {
            LowpassFilter.SetFrequency(LPFParam.Update());
        }
        
        float FloatSample = ((float) IntSample) / 32767.0f;
        if(LPFParam.GetValue() != MAX_FILTER_FREQUENCY)
        {
            FloatSample = LowpassFilter.ProcessAudioSample(FloatSample);
        }
        *OutData++ = (SInt16) (FloatSample * 32767.0f);
	}
	
	if(Source->bChannel0Finished && Channel == Source->IOSBuffer->NumChannels - 1)
	{
		Source->bAllChannelsFinished = true;
	}
	
	UnlockCallback(&Source->CallbackLock);
	
	return noErr;
}

