// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IOSAudioDevice.h: Unreal IOSAudio audio interface object.
 =============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "AudioEffect.h"
#include "AudioDevice.h"
#include "Sound/SoundWave.h"
#include "DSP/ParamInterpolator.h"
#include "DSP/OnePole.h"

/*------------------------------------------------------------------------------------
	Audio Framework system headers
 ------------------------------------------------------------------------------------*/
#include <AudioToolbox/AudioToolbox.h>
#include <AudioUnit/AudioUnit.h>
#include <AVFoundation/AVAudioSession.h>

#define AudioSampleType SInt16

DECLARE_LOG_CATEGORY_EXTERN(LogIOSAudio, Log, All);

/*------------------------------------------------------------------------------------
	Dependencies, helpers & forward declarations.
------------------------------------------------------------------------------------*/

#define CHANNELS_PER_BUS 2
#define AudioCallbackFrameSize	(8 * 1024)	// This needs to be known ahead of time to allocate appropriate buffers

struct FWaveInstance;
class FIOSAudioDevice;
class FIOSAudioEffectsManager;
class ICompressedAudioInfo;

enum ESoundFormat
{
	SoundFormat_Invalid,
	SoundFormat_LPCM,
	SoundFormat_ADPCM
};

/**
 * IOSAudio implementation of FSoundBuffer, containing the wave data and format information.
 */
class FIOSAudioSoundBuffer : public FSoundBuffer
{
public:
	FIOSAudioSoundBuffer(FIOSAudioDevice* InAudioDevice, USoundWave* InWave, bool InStreaming, bool InProcedural);
	virtual ~FIOSAudioSoundBuffer(void);

	/**
	 * Static function used to create a buffer.
	 *
	 * @param InWave USoundWave to use as template and wave source
	 * @param AudioDevice audio device to attach created buffer to
	 * @return FIOSAudioSoundBuffer pointer if buffer creation succeeded, NULL otherwise
	 */
	static FIOSAudioSoundBuffer* Init(FIOSAudioDevice* IOSAudioDevice, USoundWave* InWave);
	
	/**
	 * Returns the size of this buffer in bytes.
	 *
	 * @return Size in bytes
	 */
	virtual int32 GetSize(void) override;
	
	// These are used by the streaming engine to manage loading/unloading of chunks
	virtual int32 GetCurrentChunkIndex() const override;
	virtual int32 GetCurrentChunkOffset() const override;

	/**
	* Read the compressed info (i.e. parse header and open up a file handle) from the given sound wave.
	*/
	virtual bool ReadCompressedInfo(USoundWave* SoundWave) override;

	/**
	 * Decompresses a chunk of compressed audio to the destination memory
	 *
	 * @param Destination		Memory to decompress to
	 * @param bLooping			Whether to loop the sound seamlessly, or pad with zeroes
	 * @return					Whether the sound looped or not
	 */
	virtual bool ReadCompressedData( uint8* Destination, int32 NumFramesToDecode, bool bLooping ) override;
	
	bool ReleaseCurrentChunk();
	
	int32  RenderCallbackBufferSize;
	int32  SampleRate;
	int32  SoundFormat;
	int16* SampleData;
	uint32 BufferSize;
	
	/** Wrapper to assist in the bookkeeping of uncompressed data when streaming */
	ICompressedAudioInfo*		DecompressionState;
	bool	bStreaming;
	bool    bIsProcedural;
};

/**
 * IOSAudio implementation of FSoundSource, the interface used to play, stop and update sources
 */
class FIOSAudioSoundSource : public FSoundSource
{
public:
	/**
	 * Constructor
	 *
	 * @param	InAudioDevice	audio device this source is attached to
	 */
	FIOSAudioSoundSource(FIOSAudioDevice* InAudioDevice, uint32 InBusNumber);
	
	/** 
	 * Destructor
	 */
	~FIOSAudioSoundSource(void);
	
	/**
	 * Initializes a source with a given wave instance and prepares it for playback.
	 *
	 * @param	WaveInstance	wave instance being primed for playback
	 * @return	true			if initialization was successful, false otherwise
	 */
	virtual bool Init(FWaveInstance* WaveInstance) override;
	
	/**
	 * Updates the source specific parameter like e.g. volume and pitch based on the associated
	 * wave instance.	
	 */
	virtual void Update(void) override;
	
	/**  Plays the current wave instance. */
	virtual void Play(void) override;
	
	/** Stops the current wave instance and detaches it from the source. */
	virtual void Stop(void) override;
	
	/** Pauses playback of current wave instance. */
	virtual void Pause(void) override;
	
	/**
	 * Queries the status of the currently associated wave instance.
	 *
	 * @return	true if the wave instance/ source has finished playback and false if it is 
	 *			currently playing or paused.
	 */
	virtual bool IsFinished(void) override;

	/** Calculates the audio unit element of the input channel relative to the base bus number */
	AudioUnitElement GetAudioUnitElement(int32 Channel);

    TArray<Audio::FOnePoleLPF> LowpassFilterBank;
    TArray<Audio::FParam> LPFParamBank;
    
    int32 SampleRate;
    
    // Cached version of Source->LPFFrequency.
    float SourceLPFFrequency;
    
protected:
	bool AttachToAUGraph();
	bool DetachFromAUGraph();

	FIOSAudioDevice*      IOSAudioDevice;
	
	/** Cached sound buffer associated with currently bound wave instance. */
	/** Do not shadow the declaration of Buffer in the parent class since that member gets used by the streaming engine */
	FIOSAudioSoundBuffer* IOSBuffer;
	uint32                BusNumber;
	
	int32	CallbackLock;
	
	bool	bChannel0Finished;
	bool	bAllChannelsFinished;

private:
	
	static OSStatus IOSAudioRenderCallback(void *InRefCon, AudioUnitRenderActionFlags *IOActionFlags,
	                                       const AudioTimeStamp *InTimeStamp, UInt32 InBusNumber,
	                                       UInt32 InNumberFrames, AudioBufferList *IOData);

	void CleanupAudioBuffer();

	friend class FIOSAudioDevice;
};

/**
 * IOSAudio implementation of an Unreal audio device.
 */
class FIOSAudioDevice : public FAudioDevice
{
public:
	FIOSAudioDevice();
	virtual ~FIOSAudioDevice() { }		

	AUGraph GetAudioUnitGraph() const { return AudioUnitGraph; }
	AUNode GetMixerNode() const { return MixerNode; }
	AudioUnit GetMixerUnit() const { return MixerUnit; }
	AudioUnit GetOutputUnit() const { return OutputUnit; }

	/** Thread context management */
	virtual void ResumeContext();
	virtual void SuspendContext();
	
	virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar = *GLog ) override;

	virtual void UpdateDeviceDeltaTime() override;
    
    static void IncrementSuspendCounter();
    static void DecrementSuspendCounter();

protected:
	/** Starts up any platform specific hardware/APIs */
	virtual bool InitializeHardware() override;

	/** Shuts down any platform specific hardware/APIs */
	virtual void TeardownHardware() override;

	/** Lets the platform any tick actions */
	virtual void UpdateHardware() override;

	/** Creates a new platform specific sound source */
	virtual FAudioEffectsManager* CreateEffectsManager() override;

	/** Creates a new platform specific sound source */
	virtual FSoundSource* CreateSoundSource() override;

	/** Audio Session management */
	void GetHardwareSampleRate(double& OutSampleRate);
	bool SetHardwareSampleRate(const double& InSampleRate);
	bool SetAudioSessionActive(bool bActive);

	/** Check if any background music or sound is playing through the audio device */
	virtual bool IsExernalBackgroundSoundActive() override;

	/** Retrieves the platform settings */
	virtual FAudioPlatformSettings GetPlatformSettings() const override;

private:

	// Used to support adpcm streaming
	virtual bool HasCompressedAudioInfoClass(USoundWave* SoundWave) override { return true; }	

	void HandleError(const TCHAR* InLogOutput, bool bTeardown = false);

	AudioStreamBasicDescription MixerFormat;
	AUGraph                     AudioUnitGraph;
	AUNode                      OutputNode;
	AudioUnit                   OutputUnit;
	AUNode                      MixerNode;
	AudioUnit                   MixerUnit;
	FVector                     PlayerLocation;
	FVector                     PlayerFacing;
	FVector                     PlayerUp;
	FVector                     PlayerRight;
	int32                       NextBusNumber;

	friend class FIOSAudioSoundBuffer;
	friend class FIOSAudioSoundSource;
};
