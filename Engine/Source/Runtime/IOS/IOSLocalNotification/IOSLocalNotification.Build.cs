// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IOSLocalNotification : ModuleRules
{
	public IOSLocalNotification(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		BinariesSubFolder = "IOS";

		PublicIncludePaths.AddRange(new string[]
		{
			"Runtime/IOS/IOSLocalNotification/Public",
			"Runtime/Engine/Public",
		});
		
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
		});
	}
}
