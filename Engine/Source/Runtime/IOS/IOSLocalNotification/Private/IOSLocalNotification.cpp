// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 	IOSLocalNotification.cpp: Unreal IOSLocalNotification service interface object.
 =============================================================================*/

/*------------------------------------------------------------------------------------
	Includes
 ------------------------------------------------------------------------------------*/

#include "IOSLocalNotification.h"

#include "Apple/AppleStringUtils.h"
#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

#include "IOS/IOSApplication.h"
#include "IOS/IOSAppDelegate.h"

#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"

#import <UserNotifications/UserNotifications.h>

DEFINE_LOG_CATEGORY(LogIOSLocalNotification);

class FIOSLocalNotificationModule : public ILocalNotificationModule
{
public:

	/** Creates a new instance of the audio device implemented by the module. */
	virtual ILocalNotificationService* GetLocalNotificationService() override
	{
		static ILocalNotificationService*	oneTrueLocalNotificationService = nullptr;
		
		if(oneTrueLocalNotificationService == nullptr)
		{
			oneTrueLocalNotificationService = new FIOSLocalNotificationService;
		}
		
		return oneTrueLocalNotificationService;
	}

#if !PLATFORM_TVOS
	static UNMutableNotificationContent* CreateNotificationContent(NSString* NotificationTitle, NSString* NotificationBody, NSString* ActivationEventString, uint32 BadgeNumber)
	{
		UNMutableNotificationContent* Content = [UNMutableNotificationContent new];
		if(Content != nil)
		{
			if(NotificationTitle != nil)
			{
				Content.title = NotificationTitle;
			}

			if(NotificationBody != nil)
			{
				Content.body = NotificationBody;
			}

			NSNumber* BadgeNSNumber = [NSNumber numberWithInt:BadgeNumber];
			Content.badge = BadgeNSNumber;
			Content.sound = [UNNotificationSound defaultSound];

			if(ActivationEventString != nil)
			{
				NSDictionary* Dict = [NSDictionary dictionaryWithObjectsAndKeys: ActivationEventString, @"ActivationEvent", @"Local", @"NotificationType", nil];
				if (Dict != nil)
				{
					Content.userInfo = Dict;
				}
			}
		}

		return Content;
	}
	static UNCalendarNotificationTrigger* CreateCalendarNotificationTrigger(const FDateTime& FireDateTime, bool LocalTime)
	{
		NSCalendar *calendar = [NSCalendar autoupdatingCurrentCalendar];
		NSDateComponents *dateComps = [[NSDateComponents alloc] init];
		[dateComps setDay : FireDateTime.GetDay()];
		[dateComps setMonth : FireDateTime.GetMonth()];
		[dateComps setYear : FireDateTime.GetYear()];
		[dateComps setHour : FireDateTime.GetHour()];
		[dateComps setMinute : FireDateTime.GetMinute()];
		[dateComps setSecond : FireDateTime.GetSecond()];
		if (!LocalTime)
		{
			// if not local time, convert from UTC to local time, UNCalendarNotificationTrigger misbehaves
			// if the components have anything more than previously set (can't specify timeZone)
			NSCalendar *utcCalendar = [NSCalendar calendarWithIdentifier:NSCalendarIdentifierISO8601];
			utcCalendar.timeZone = [NSTimeZone timeZoneForSecondsFromGMT:0];
			NSDate *utcDate = [utcCalendar dateFromComponents:dateComps];
			NSDateComponents *utcDateComps = [utcCalendar componentsInTimeZone:calendar.timeZone fromDate:utcDate];
			
			// now copy components back because utcDateComps has too many fields set for
			// UNCalendarNotificationTrigger and will now work properly on all devices
			dateComps.day = utcDateComps.day;
			dateComps.month = utcDateComps.month;
			dateComps.year = utcDateComps.year;
			dateComps.hour = utcDateComps.hour;
			dateComps.minute = utcDateComps.minute;
			dateComps.second = utcDateComps.second;
		}
		
		UNCalendarNotificationTrigger *trigger = [UNCalendarNotificationTrigger triggerWithDateMatchingComponents:dateComps repeats:NO];
		
		[dateComps release];
		return trigger;
	}
#endif
};

IMPLEMENT_MODULE(FIOSLocalNotificationModule, IOSLocalNotification);

/*------------------------------------------------------------------------------------
	FIOSLocalNotification
 ------------------------------------------------------------------------------------*/
FIOSLocalNotificationService::FIOSLocalNotificationService()
{
	AppLaunchedWithNotification = false;
	LaunchNotificationFireDate = 0;
}

void FIOSLocalNotificationService::ClearAllLocalNotifications()
{
#if !PLATFORM_TVOS
	dispatch_async(dispatch_get_main_queue(), ^{
		UNUserNotificationCenter *Center = [UNUserNotificationCenter currentNotificationCenter];
		[Center removeAllPendingNotificationRequests];
	});
#endif
}

static int32 NotificationNumber = 0;
int32 FIOSLocalNotificationService::ScheduleLocalNotificationAtTime(const FDateTime& FireDateTime, bool LocalTime, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent)
{
#if !PLATFORM_TVOS
	const int32 CurrentNotificationId = NotificationNumber++;
	return ScheduleLocalNotificationAtTimeOverrideId(FireDateTime, LocalTime, Title, Body, Action, ActivationEvent, CurrentNotificationId);
#else
	return -1;
#endif
}

int32 FIOSLocalNotificationService::ScheduleLocalNotificationAtTimeOverrideId(const FDateTime& FireDateTime, bool LocalTime, const FText& Title, const FText& Body, const FText& Action, const FString& ActivationEvent, int32 IdOverride)
{
#if !PLATFORM_TVOS
	if (FireDateTime < (LocalTime ? FDateTime::Now() : FDateTime::UtcNow()))
	{
		return -1;
	}

	//Convert FText/FString to NSString on the calling thread (which has the game thread task tag scope)
	//before dispatching to the GCD main queue where UE thread tags are not set.
	NSString* NotificationTitle = Title.IsEmpty() ? nil : [NSString stringWithFString:Title.ToString()];
	NSString* NotificationBody = Body.IsEmpty() ? nil : [NSString stringWithFString:Body.ToString()];
	NSString* ActivationEventString = ActivationEvent.IsEmpty() ? nil : [NSString stringWithFString:ActivationEvent];
	FDateTime FireDateTimeCopy = FireDateTime;
	const int32 CurrentNotificationId = IdOverride;

	//have to schedule notification on main thread queue
	dispatch_async(dispatch_get_main_queue(), ^{
		UNMutableNotificationContent* Content = FIOSLocalNotificationModule::CreateNotificationContent(NotificationTitle, NotificationBody, ActivationEventString, 1);
		UNCalendarNotificationTrigger* Trigger = FIOSLocalNotificationModule::CreateCalendarNotificationTrigger(FireDateTimeCopy, LocalTime);
		
		UNNotificationRequest *Request = [UNNotificationRequest requestWithIdentifier:@(CurrentNotificationId).stringValue content:Content trigger:Trigger];
		
		UNUserNotificationCenter *Center = [UNUserNotificationCenter currentNotificationCenter];
		[Center addNotificationRequest : Request withCompletionHandler : ^ (NSError * _Nullable error) {
			if (error != nil) {
				UE_LOGF(LogIOSLocalNotification, Warning, "Error scheduling notification: %d", CurrentNotificationId);
			}
		}];
		[Content release];
	});
	return CurrentNotificationId;
#else
	return -1;
#endif
}

int32 FIOSLocalNotificationService::ScheduleLocalNotificationBadgeAtTime(const FDateTime& FireDateTime, bool LocalTime, const FString& ActivationEvent)
{
#if !PLATFORM_TVOS
	if (FireDateTime < (LocalTime ? FDateTime::Now() : FDateTime::UtcNow()))
	{
		return -1;
	}

	NSString* ActivationEventString = ActivationEvent.IsEmpty() ? nil : [NSString stringWithFString:ActivationEvent];
	FDateTime FireDateTimeCopy = FireDateTime;
	int32 CurrentNotificationId = NotificationNumber++;

	//have to schedule notification on main thread queue
	dispatch_async(dispatch_get_main_queue(), ^{
		UNMutableNotificationContent* Content = FIOSLocalNotificationModule::CreateNotificationContent(nil, nil, ActivationEventString, 1);
		UNCalendarNotificationTrigger* Trigger = FIOSLocalNotificationModule::CreateCalendarNotificationTrigger(FireDateTime, LocalTime);
		
		UNNotificationRequest *Request = [UNNotificationRequest requestWithIdentifier:@(CurrentNotificationId).stringValue content:Content trigger:Trigger];

		UNUserNotificationCenter *Center = [UNUserNotificationCenter currentNotificationCenter];
		[Center addNotificationRequest:Request withCompletionHandler:^(NSError * _Nullable error) {
			if (error != nil) {
				UE_LOGF(LogIOSLocalNotification, Warning, "Error scheduling notification: %d", CurrentNotificationId);
			}
		}];
		[Content release];
	});
	
	return CurrentNotificationId;
#else
	return -1;
#endif
}

void FIOSLocalNotificationService::CancelLocalNotification(const FString& ActivationEvent)
{
	// TODO
}

void FIOSLocalNotificationService::CancelLocalNotification(int32 NotificationId)
{
#if !PLATFORM_TVOS
	UNUserNotificationCenter *Center = [UNUserNotificationCenter currentNotificationCenter];
	NSArray<NSString*> *Identifiers = @[@(NotificationId).stringValue];
	[Center removePendingNotificationRequestsWithIdentifiers: Identifiers];
	[Center removeDeliveredNotificationsWithIdentifiers: Identifiers];
#endif
}

void FIOSLocalNotificationService::GetLaunchNotification(bool& NotificationLaunchedApp, FString& ActivationEvent, int32& FireDate)
{
	NotificationLaunchedApp = AppLaunchedWithNotification;
	ActivationEvent = LaunchNotificationActivationEvent;
	FireDate = LaunchNotificationFireDate;
}

void FIOSLocalNotificationService::SetLaunchNotification(FString const& ActivationEvent, int32 FireDate)
{
	AppLaunchedWithNotification = true;
	LaunchNotificationActivationEvent = ActivationEvent;
	LaunchNotificationFireDate = FireDate;
}


static FIOSLocalNotificationService::FAllowedNotifications NotificationsAllowedDelegate;
void FIOSLocalNotificationService::CheckAllowedNotifications(const FAllowedNotifications& AllowedNotificationsDelegate)
{
	NotificationsAllowedDelegate = AllowedNotificationsDelegate;
	
#if !PLATFORM_TVOS
	UNUserNotificationCenter *Center = [UNUserNotificationCenter currentNotificationCenter];
	[Center getNotificationSettingsWithCompletionHandler:^(UNNotificationSettings * _Nonnull settings) {
		bool NotificationsAllowed = settings.authorizationStatus == UNAuthorizationStatusAuthorized;
		NotificationsAllowedDelegate.ExecuteIfBound(NotificationsAllowed);
	}];
#else
	NotificationsAllowedDelegate.ExecuteIfBound(false);
#endif
}
