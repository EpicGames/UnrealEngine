// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSGamepadUtils.h"
#include "GameDelegates.h"
#include "IOS/IOSApplication.h"
#include "IOS/IOSPlatformApplicationMisc.h"
#include "TextureResource.h"

DEFINE_LOG_CATEGORY_STATIC(LogIOSGamepadUtils, Log, All);

//
// Implementation members
//

FIOSGamepadUtils::FIOSGamepadUtils()
{
    // Bind FIOSPlatformApplicationMisc's GetGamePadGlyph delegate to us.
	FIOSPlatformApplicationMisc::GetGamePadGlyphDelegate.BindRaw(this, &FIOSGamepadUtils::GetGamepadButtonGlyph);
}

FIOSGamepadUtils::~FIOSGamepadUtils()
{
	FIOSPlatformApplicationMisc::GetGamePadGlyphDelegate.Unbind();
}

UTexture2D* FIOSGamepadUtils::GetGamepadButtonGlyph(const FGamepadKeyNames::Type& ButtonKey, uint32 ControllerIndex)
{
    UTexture2D *NewTexture = nullptr;

    FIOSInputInterface* InputInterface = static_cast<FIOSInputInterface*>(FIOSPlatformApplicationMisc::CachedApplication->GetInputInterface());
    check(InputInterface);

    ControllerType ControllerType = InputInterface->GetControllerType(ControllerIndex);

    for (int32 Index = 0; Index != GlyphsArray.Num(); ++Index)
    {
        if (GlyphsArray[Index].ButtonName == ButtonKey && GlyphsArray[Index].ControllerType == ControllerType)
        {
            return GlyphsArray[Index].ButtonTexture;
            
        }

        NSData* RawData = InputInterface->GetGamepadGlyphRawData(ButtonKey, ControllerIndex);
        UInt8 Buf[RawData.length];
        [RawData getBytes:Buf length:RawData.length];
        const uint8_t* Bytes = (const uint8_t*)[RawData bytes];
        long Length = [RawData length];
        IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
        TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);

        if (ImageWrapper != nullptr && Length > 0)
        {
            TArray<uint8> RawImageData;
            ImageWrapper->SetCompressed(Bytes, Length);

                if (ImageWrapper->GetRaw(ERGBFormat::BGRA, 8, RawImageData))
                {
                    NewTexture = UTexture2D::CreateTransient(ImageWrapper->GetWidth(), ImageWrapper->GetHeight(), PF_B8G8R8A8);

                    if (NewTexture != nullptr)
                    {					
						FTexture2DMipMap& Mip0 = NewTexture->GetPlatformData()->Mips[0];
						void* TextureData = Mip0.BulkData.Lock(LOCK_READ_WRITE);
                        FMemory::Memcpy(TextureData, RawImageData.GetData(), RawImageData.Num());
						Mip0.BulkData.Unlock();
                        NewTexture->UpdateResource();
                        GamepadGlyph GamepadGlyph;
                        GamepadGlyph.ControllerType = ControllerType;
                        GamepadGlyph.ButtonName = ButtonKey;
                        GamepadGlyph.ButtonTexture = NewTexture;
                        GamepadGlyph.ButtonTexture->AddToRoot();
                        GlyphsArray.Add(GamepadGlyph);
                    }
                }
        }
    }
    return NewTexture;
}
