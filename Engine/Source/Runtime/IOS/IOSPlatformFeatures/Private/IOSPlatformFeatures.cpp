// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "PlatformFeatures.h"
#include "IOSSaveGameSystem.h"
#include "IOSGamepadUtils.h"

class FIOSPlatformFeatures : public IPlatformFeaturesModule
{
public:
	virtual void StartupModule() override
	{
		// create the persistent gamepad utils
		IOSGamepadUtils = new FIOSGamepadUtils;
	}

	virtual void ShutdownModule() override
	{
		delete IOSGamepadUtils;
		IOSGamepadUtils = nullptr;
	}

	virtual class ISaveGameSystem* GetSaveGameSystem() override
	{
		static FIOSSaveGameSystem IOSSaveGameSystem;
		return &IOSSaveGameSystem;
	}
    
private:
	FIOSGamepadUtils* IOSGamepadUtils = nullptr;
};

IMPLEMENT_MODULE(FIOSPlatformFeatures, IOSPlatformFeatures);
