// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSSaveGameSystem.h"
#include "GameDelegates.h"

DEFINE_LOG_CATEGORY_STATIC(LogIOSSaveGame, Log, All);

//
// Implementation members
//

FIOSSaveGameSystem::FIOSSaveGameSystem()
{
	Initialize();
}

FIOSSaveGameSystem::~FIOSSaveGameSystem()
{
	Shutdown();
}

void FIOSSaveGameSystem::Initialize()
{
}

void FIOSSaveGameSystem::Shutdown()
{
}

bool FIOSSaveGameSystem::GetSaveGameNames(TArray<FString>& FoundSaves, const int32 UserIndex)
{
	TArray<FString> FoundFiles;
	const FString SaveGameDirectory = FPaths::ProjectSavedDir() / TEXT("SaveGames/");
	IFileManager::Get().FindFiles(FoundFiles, *SaveGameDirectory, TEXT("*.sav"));

	for (FString& File : FoundFiles)
	{
		FoundSaves.Add(FPaths::GetBaseFilename(File));
	}

	return true;
}

ISaveGameSystem::ESaveExistsResult FIOSSaveGameSystem::DoesSaveGameExistWithResult(const TCHAR* Name, const int32 UserIndex)
{
	if (IFileManager::Get().FileSize(*GetSaveGamePath(Name)) >= 0)
	{
		return ESaveExistsResult::OK;
	}
	return ESaveExistsResult::DoesNotExist;
}

bool FIOSSaveGameSystem::SaveGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex, const TArray<uint8>& Data)
{
    bool DidSucceed = false;
    
    if (OnUpdateCloudDataFromLocalSave.IsBound())
    {
        DidSucceed = OnUpdateCloudDataFromLocalSave.Execute(FString::Printf(TEXT("%s""SaveGames/%s.sav"), *FPaths::ProjectSavedDir(), Name));
    }

    return DidSucceed ? DidSucceed : FFileHelper::SaveArrayToFile(Data, *GetSaveGamePath(Name));
}

bool FIOSSaveGameSystem::LoadGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex, TArray<uint8>& Data)
{
    bool DidSucceed = false;

    if (OnUpdateLocalSaveFileFromCloud.IsBound())
    {
        DidSucceed = OnUpdateLocalSaveFileFromCloud.Execute(FString::Printf(TEXT("%s""SaveGames/%s.sav"), *FPaths::ProjectSavedDir(), Name));
    }
    
    return DidSucceed ? DidSucceed : FFileHelper::LoadFileToArray(Data, *GetSaveGamePath(Name));
}

bool FIOSSaveGameSystem::DeleteGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex)
{
	// delete the file from the iCloud
    OnDeleteCloudData.ExecuteIfBound(FString(Name));

	// delete the file from the local storage
	return IFileManager::Get().Delete(*GetSaveGamePath(Name), true, false, !bAttemptToUseUI);
}

FString FIOSSaveGameSystem::GetSaveGamePath(const TCHAR* Name)
{
	return FString::Printf(TEXT("%s""SaveGames/%s.sav"), *FPaths::ProjectSavedDir(), Name);
}
