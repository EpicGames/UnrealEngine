// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageUtils.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "IOS/IOSInputInterface.h"
#include "Modules/ModuleManager.h"
#include "Engine/Texture2D.h"

class FIOSGamepadUtils
{
public:

    struct GamepadGlyph
    {
        ControllerType ControllerType;
        FGamepadKeyNames::Type ButtonName;
        UTexture2D* ButtonTexture;
    };
	FIOSGamepadUtils();
	virtual ~FIOSGamepadUtils();

    UTexture2D* GetGamepadButtonGlyph(const FGamepadKeyNames::Type& ButtonKey, uint32 ControllerIndex);
    
private:
    TArray<GamepadGlyph> GlyphsArray;
};
