// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SaveGameSystem.h"

class FIOSSaveGameSystem : public ISaveGameSystem
{
public:
	FIOSSaveGameSystem();
	virtual ~FIOSSaveGameSystem();

	// ISaveGameSystem interface
	virtual bool PlatformHasNativeUI() override
	{
		return false;
	}

	virtual bool DoesSaveSystemSupportMultipleUsers() override
	{
		return false;
	}

	virtual bool GetSaveGameNames(TArray<FString>& FoundSaves, const int32 UserIndex) override;

	virtual bool DoesSaveGameExist(const TCHAR* Name, const int32 UserIndex) override
	{
		return ESaveExistsResult::OK == DoesSaveGameExistWithResult(Name, UserIndex);
	}

	virtual ESaveExistsResult DoesSaveGameExistWithResult(const TCHAR* Name, const int32 UserIndex) override;
	
	virtual bool SaveGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex, const TArray<uint8>& Data) override;
	 /**
     * We call this to notify any listening Cloud subsystem that we have updated a save file.
     */
    DECLARE_DELEGATE_RetVal_OneParam(bool, FUpdateCloudDataFromLocalSave, const FString&);
    FUpdateCloudDataFromLocalSave OnUpdateCloudDataFromLocalSave;

	virtual bool LoadGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex, TArray<uint8>& Data) override;
	/**
     * We call this to refresh a save file from a listening Cloud subsystem, before we open it.
     */
    DECLARE_DELEGATE_RetVal_OneParam(bool, FUpdateLocalSaveFileFromCloud, const FString&);
    FUpdateLocalSaveFileFromCloud OnUpdateLocalSaveFileFromCloud;

	virtual bool DeleteGame(bool bAttemptToUseUI, const TCHAR* Name, const int32 UserIndex) override;
	/**
     * We call this to delete a save file from a listening Cloud subsystem.
     */
	DECLARE_DELEGATE_OneParam(FDeleteCloudData, const FString&);
	FDeleteCloudData OnDeleteCloudData;

private:
	/**
	 * Initializes the SaveData library then loads and initializes the SaveDialog library
	 */
	void Initialize();

	/**
	 * Terminates and unloads the SaveDialog library then terminates the SaveData library
	 */
	void Shutdown();

	/**
	 * Get the path to save game file for the given name
	 */
	virtual FString GetSaveGamePath(const TCHAR* Name);
};
