// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/PropertyPortFlags.h"
#include "AudioCompressionSettings.h"

#include "IOSRuntimeSettings.generated.h"

UENUM()
enum class EPowerUsageFrameRateLock : uint8
{
    /** Frame rate is not limited. */
    PUFRL_None = 0 UMETA(DisplayName="None"),
        
    /** Frame rate is limited to a maximum of 20 frames per second. */
    PUFRL_20 = 20 UMETA(DisplayName="20 FPS"),
    
    /** Frame rate is limited to a maximum of 30 frames per second. */
    PUFRL_30 = 30 UMETA(DisplayName="30 FPS"),
    
    /** Frame rate is limited to a maximum of 60 frames per second. */
    PUFRL_60 = 60 UMETA(DisplayName="60 FPS"),
};

UENUM()
	enum class EIOSVersion : uint8
{
    IOS_Minimum = 15 UMETA(DisplayName = "Minimum, Currently 15.0"),
    IOS_15 = 15 UMETA(DisplayName = "15.0"),
    IOS_16 = 16 UMETA(DisplayName = "16.0"),
    IOS_17 = 17 UMETA(DisplayName = "17.0"),
    IOS_18 = 18 UMETA(DisplayName = "18.0"),
    IOS_26 = 26 UMETA(DisplayName = "26.0"),
};

// https://support.apple.com/en-ca/HT205073
UENUM()
enum class EIOSMetalShaderStandard : uint8
{
    /** Metal Shader 2.4 is the minimum as of UE5.3*/
    IOSMetalSLStandard_Minimum = 0 UMETA(DisplayName="Minimum, Metal v2.4"),
    /** Metal Shaders Compatible With iOS 16.0/tvOS 16.0 or later (std=metal2.4) */
    IOSMetalSLStandard_2_4 = 7 UMETA(DisplayName="Metal v2.4 (iOS 15.0/tvOS 15.0 for older devices)"),
    /** Metal Shaders Compatible With iOS 16.0/tvOS 16.0 or later (std=metal3.0) */
    IOSMetalSLStandard_3_0 = 8 UMETA(DisplayName="Metal v3.0 (iOS 16.0/tvOS 16.0)"),
    /** Metal Shaders Compatible With iOS 17.0/tvOS 17.0 or later (std=metal3.1) */
    IOSMetalSLStandard_3_1 = 9 UMETA(DisplayName="Metal v3.1 (iOS 17.0/tvOS 17.0)"),

};

UENUM()
enum class EIOSLandscapeOrientation : uint8
{
	/** Landscape Left */
	LandscapeLeft = 0 UMETA(DisplayName = "Landscape (left home button)"),

	/** Landscape Right */
	LandscapeRight = 1 UMETA(DisplayName = "Landscape (right home button)"),
};

UENUM()
enum class EIOSCloudKitSyncStrategy : uint8
{
	/** Only at game start */
	None  = 0 UMETA(DisplayName = "Never (do not use iCloud for Load/Save Game)"),

	/** Only at game start */
	OnlyAtGameStart = 1 UMETA(DisplayName = "At game start only (iOS)"),

	/** Always */
	Always = 2 UMETA(DisplayName = "Always (whenever LoadGame is called)"),
};

/**
 *	Screen edges where the app suppresses iOS system gestures (home indicator, notification
 *	center, control center) on the first swipe so the app receives the touch. Selected edges
 *	require the user to swipe twice before the system gesture fires.
 */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EIOSScreenEdge : uint8
{
	Top    = 1 << 0 UMETA(DisplayName = "Top"),
	Left   = 1 << 1 UMETA(DisplayName = "Left"),
	Bottom = 1 << 2 UMETA(DisplayName = "Bottom"),
	Right  = 1 << 3 UMETA(DisplayName = "Right"),
};
ENUM_CLASS_FLAGS(EIOSScreenEdge);

/**
 *	IOS Build resource file struct, used to serialize filepaths to the configs for use in the build system,
 */
USTRUCT()
struct FIOSBuildResourceFilePath
{
	GENERATED_USTRUCT_BODY()
	
	/**
	 * Custom export item used to serialize FIOSBuildResourceFilePath types as only a filename, no garland.
	 */
	bool ExportTextItem(FString& ValueStr, FIOSBuildResourceFilePath const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
	{
		ValueStr += FilePath;
		return true;
	}

	/**
	 * Custom import item used to parse ini entries straight into the filename.
	 */
	bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
	{
		FilePath = Buffer;
		return true;
	}

	/**
	 * The path to the file.
	 */
	UPROPERTY(EditAnywhere, Category = FilePath)
	FString FilePath;
};

/**
 *	Setup our resource filepath to make it easier to parse in UBT
 */
template<>
struct TStructOpsTypeTraits<FIOSBuildResourceFilePath> : public TStructOpsTypeTraitsBase2<FIOSBuildResourceFilePath>
{
	enum
	{
		WithExportTextItem = true,
		WithImportTextItem = true,
	};
};



/**
 *	IOS Build resource file struct, used to serialize Directorys to the configs for use in the build system,
 */
USTRUCT()
struct FIOSBuildResourceDirectory
{
	GENERATED_USTRUCT_BODY()

	/**
	 * Custom export item used to serialize FIOSBuildResourceDirectory types as only a filename, no garland.
	 */
	bool ExportTextItem(FString& ValueStr, FIOSBuildResourceDirectory const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
	{
		ValueStr += Path;
		return true;
	}

	/**
	 * Custom import item used to parse ini entries straight into the filename.
	 */
	bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
	{
		Path = Buffer;
		return true;
	}

	/**
	* The path to the file.
	*/
	UPROPERTY(EditAnywhere, Category = Directory)
	FString Path;
};

/**
*	Setup our resource Directory to make it easier to parse in UBT
*/
template<>
struct TStructOpsTypeTraits<FIOSBuildResourceDirectory> : public TStructOpsTypeTraitsBase2<FIOSBuildResourceDirectory>
{
	enum
	{
		WithExportTextItem = true,
		WithImportTextItem = true,
	};
};



/**
 * Implements the settings for the iOS target platform.
 */
UCLASS(config=Engine, defaultconfig)
class IOSRUNTIMESETTINGS_API UIOSRuntimeSettings : public UObject
{
public:
	GENERATED_UCLASS_BODY()

	// Should StoreKit support be enabled?
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Online, meta = (ConfigHierarchyEditable))
	bool bEnableStoreKitSupport;

	// Should Game Center support (iOS Online Subsystem) be enabled?
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Online, meta = (ConfigHierarchyEditable))
    bool bEnableGameCenterSupport;
	
	// Should Cloud Kit support (iOS Online Subsystem) be enabled?
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Online)
	bool bEnableCloudKitSupport;

	// iCloud Read stategy
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Online, meta = (DisplayName = "iCloud save files sync strategy"), meta = (EditCondition = "bEnableCloudKitSupport"))
	EIOSCloudKitSyncStrategy IOSCloudKitSyncStrategy;

    // Should push/remote notifications support (iOS Online Subsystem) be enabled?
    UPROPERTY(GlobalConfig, EditAnywhere, Category = Online)
    bool bEnableRemoteNotificationsSupport;
    
    // Should background fetch support be enabled?
    UPROPERTY(GlobalConfig, EditAnywhere, Category = Online)
    bool bEnableBackgroundFetch;
    
	// Whether or not to compile iOS Metal shaders for the Mobile renderer (requires iOS 8+ and an A7 processor).
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Rendering, meta = (DisplayName = "Metal Mobile Renderer"))
	bool bSupportsMetal;

	// Whether or not to compile iOS Metal shaders for the SM5 renderer (requires iOS 10+ and an A10 processor)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Rendering, meta = (DisplayName = "Metal Mobile SM5 Renderer"))
	bool bSupportsMetalMobileSM5;

	// Whether or not to compile iOS Metal shaders for the SM6 renderer (requires iOS 15+ and an A17 Pro processor or M2)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Rendering, meta = (DisplayName = "Metal Mobile SM6 Renderer (Experimental)"))
	bool bSupportsMetalMobileSM6;

	// Enable ray tracing support for Metal SM6 (requires iOS 18.0+ and an A17 Pro or M2 device)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Rendering, meta = (DisplayName = "Enable Ray Tracing (SM6 Experimental)", EditCondition = "bSupportsMetalMobileSM6"))
	bool bEnableRayTracing = false;

    // Should the app be compatible for high refresh rate (iPhone only)
    UPROPERTY(GlobalConfig, EditAnywhere, Category = Rendering, meta = (DisplayName = "Enable ProMotion 120Hz on supported iPhone devices"))
    bool bSupportHighRefreshRates;

    /** Whether to enable LOD streaming for landscape visual meshes. Requires Metal support. */
    UPROPERTY(GlobalConfig, EditAnywhere, Category = Rendering, Meta = (DisplayName = "Stream landscape visual mesh LODs"))
    bool bStreamLandscapeMeshLODs;
    
    // Minimum iOS version this game supports
    UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Minimum iOS Version"))
    EIOSVersion MinimumiOSVersion;

	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Override location of Metal toolchain"))
	FIOSBuildResourceDirectory WindowsMetalToolchainOverride;

	// Enable generation of dSYM file
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Generate dSYMs for code debugging and profiling"))
	bool bGeneratedSYMFile;

	// Enable generation of dSYM bundle
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Generate dSYMs as a bundle for third party crash tools"), meta = (EditCondition = "bGeneratedSYMFile"))
	bool bGeneratedSYMBundle;

	// Enable generation of a .udebugsymbols file, which allows offline, platform-independent symbolication for the Malloc Profiler or external crash reporting tools. Requires a dSYM file or bundle.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Generate .udebugsymbols file"))
	bool bGenerateCrashReportSymbols;
	
	// Enable generation of xcode archive package
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Generate xcode archive package"))
	bool bGenerateXCArchive;

	// Does the app require use of NON-standard encryption
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "ITSAppUsesNonExemptEncryption plist value"))
	bool bUsesNonExemptEncryption;

	// If bUsesNonExemptEncryption, set the ITSEncryptionExportComplianceCode
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (EditCondition = "bUsesNonExemptEncryption", DisplayName = "Set ITSEncryptionExportComplianceCode"))
	FString ITSEncryptionExportComplianceCode;

	// Enable Advertising Identified
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Enable Advertising Identified (IDFA)"))
	bool bEnableAdvertisingIdentifier;

	// Any additional linker flags to pass to the linker in non-shipping builds
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Additional Non-Shipping Linker Flags", ConfigHierarchyEditable))
	FString AdditionalLinkerFlags;

	// Any additional linker flags to pass to the linker in shipping builds
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Additional Shipping Linker Flags", ConfigHierarchyEditable))
	FString AdditionalShippingLinkerFlags;
    
    // Any additional plist key/value data utilizing \n for a new line
    UPROPERTY(GlobalConfig, EditAnywhere, Category = Build)
    FString AdditionalPlistData;
    
	// List of any entitlements we require. The app will not initialize if the specified entitlements are not available.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build)
	TArray<FString> RequiredEntitlements;

    // Whether or not to add support for iPad devices
    UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Supports iPad"))
    bool bSupportsIPad;

    // Whether or not to add support for iPhone devices
    UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Supports iPhone"))
    bool bSupportsIPhone;
    
    // Whether or not the iPad app supports Split View (also needed for StageManager support)
    UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Enable iPad Split View"))
    bool bEnableSplitView;
    
    // Whether or not to enable iOS 16 dynamic linker bug workaround (bug fixed in iOS 17+, not necesssary if targetting 17.0 or higher Minimum iOS Version)
    UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Enable iOS 16 Dynamic Linker Bug Workaround (Experimental)"))
    bool bEnableIOS16DynLinkerBugWAR;

    // Adopt the UIScene-based lifecycle as required by Apple.
    // When true, the engine routes per-scene lifecycle through UIWindowSceneDelegate and Info.plist
    // declares UIApplicationSceneManifest. Compile-time only - rebuild is required after changing.
    UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (DisplayName = "Use UIScene-based lifecycle (requires rebuild)"))
    bool bUseSceneBasedLifecycle = false;
    
    /** Set the maximum frame rate to save on power consumption */
    UPROPERTY(GlobalConfig, EditAnywhere, Category = PowerUsage, meta = (ConfigHierarchyEditable))
    EPowerUsageFrameRateLock FrameRateLock;

    //Whether or not to allow taking the MaxRefreshRate from the device instead of a constant (60fps) in IOSPlatformFramePacer
    UPROPERTY(GlobalConfig, EditAnywhere, Category = PowerUsage, meta = (ConfigHierarchyEditable))
    bool bEnableDynamicMaxFPS;

    // Enable the use of RSync for remote builds on a mac
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Remote Build", meta = (DisplayName = "Use RSync for building IOS", ConfigHierarchyEditable))
    bool bUseRSync;

    // The name or ip address of the remote mac which will be used to build IOS
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Remote Build", meta = (ConfigHierarchyEditable))
    FString RemoteServerName;

    // The mac users name which matches the SSH Private Key, for remote builds using RSync.
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Remote Build", meta = (EditCondition = "bUseRSync", DisplayName = "Username on Remote Server", ConfigHierarchyEditable))
    FString RSyncUsername;

    // Optional path on the remote mac where the build files will be copied. If blank, ~/UE5/Builds will be used.
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Remote Build", meta = (ConfigHierarchyEditable))
    FString RemoteServerOverrideBuildPath;
    
    // The install directory of cwrsync.
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Remote Build", meta = (EditCondition = "bUseRSync", ConfigHierarchyEditable))
    FIOSBuildResourceDirectory CwRsyncInstallPath;

    // The existing location of an SSH Key found by Unreal Engine.
    UPROPERTY(VisibleAnywhere, Category = "Remote Build", meta = (DisplayName = "Found Existing SSH permissions file"))
    FString SSHPrivateKeyLocation;

    // The path of the ssh permissions key to be used when connecting to the remote server.
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Remote Build", meta = (EditCondition = "bUseRSync", DisplayName = "Override existing SSH permissions file", ConfigHierarchyEditable))
    FIOSBuildResourceFilePath SSHPrivateKeyOverridePath;

    // Support a secondary remote Mac to support to facilitate iOS/tvOS debug ?
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Remote Build", meta = (DisplayName = "Enable Secondary remote Mac"))
    bool bSupportSecondaryMac;
    
    // The name or ip address of the remote mac which will be used to build IOS
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Remote Build", meta = (EditCondition = "bSupportSecondaryMac", ConfigHierarchyEditable))
    FString SecondaryRemoteServerName;
    
    // The secondary mac users name which matches the SSH Private Key, for remote builds using RSync.
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Remote Build", meta = (EditCondition = "bSupportSecondaryMac", DisplayName = "Username on Secondary Remote Server", ConfigHierarchyEditable))
    FString SecondaryRSyncUsername;

    // Optional path on the secondary remote mac where the build files will be copied. If blank, ~/UE5/Builds will be used.
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Remote Build", meta = (EditCondition = "bSupportSecondaryMac", ConfigHierarchyEditable))
    FString SecondaryRemoteServerOverrideBuildPath;

    // The install directory of cwrsync.
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Remote Build", meta = (EditCondition = "bSupportSecondaryMac", ConfigHierarchyEditable))
    FIOSBuildResourceDirectory SecondaryCwRsyncInstallPath;
    
    // The existing location of an SSH Key found by Unreal Engine.
    UPROPERTY(VisibleAnywhere, Category = "Remote Build", meta = (EditCondition = "bSupportSecondaryMac", DisplayName = "Found Existing SSH permissions file for Secondary Mac"))
    FString SecondarySSHPrivateKeyLocation;

    // The path of the ssh permissions key to be used when connecting to the remote server.
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Remote Build", meta = (EditCondition = "bSupportSecondaryMac", DisplayName = "Override existing SSH permissions file for Secondary Mac", ConfigHierarchyEditable))
    FIOSBuildResourceFilePath SecondarySSHPrivateKeyOverridePath;

	// Should the app be multi-users compatible on tvOS ? Requires the com.apple.developer.user-management entitlement.
    UPROPERTY(GlobalConfig, EditAnywhere, Category = "Build", meta = (DisplayName = "Support user switching on tvOS"))
    bool bUserSwitching;

    // If checked, the game will be able to handle multiple gamepads at the same time (the Siri Remote is a gamepad)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Input, meta = (DisplayName = "Multiple gamepads support"))
	bool bGameSupportsMultipleActiveControllers;

	// If checked, the Siri Remote can be rotated to landscape view
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Input, meta = (DisplayName = "Allow AppleTV Remote landscape mode"))
	bool bAllowRemoteRotation;
		
	// If checked, Bluetooth connected controllers will send input
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Input, meta = (DisplayName = "Allow MFi (Bluetooth) controllers"))
	bool bAllowControllers;

	// Block force feedback on the device when controllers are attached.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Input, meta = (DisplayName = "Block force feedback on the device when controllers are attached"))
	bool bControllersBlockDeviceFeedback;

	// Disables usage of device motion data. If application does not use motion data disabling it will improve battery life
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Input, meta = (DisplayName = "Disable Motion Controls"))
	bool bDisableMotionData;

	// Screen edges where iOS should defer its system gestures so the app receives the first swipe.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Input, meta = (DisplayName = "Screen Edges Deferred Gestures", Bitmask, BitmaskEnum = "/Script/IOSRuntimeSettings.EIOSScreenEdge"))
	int32 ScreenEdgesDeferredGestures;

	// Supports default portrait orientation. Landscape will not be supported.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DeviceOrientations)
	uint32 bSupportsPortraitOrientation : 1;

	// Supports upside down portrait orientation. Landscape will not be supported.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DeviceOrientations)
	uint32 bSupportsUpsideDownOrientation : 1;

	// Supports left landscape orientation. Portrait will not be supported.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DeviceOrientations)
	uint32 bSupportsLandscapeLeftOrientation : 1;

	// Supports right landscape orientation. Portrait will not be supported.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DeviceOrientations)
	uint32 bSupportsLandscapeRightOrientation : 1;

	// Whether files created by the app will be accessible from the iTunes File Sharing feature
	UPROPERTY(GlobalConfig, EditAnywhere, Category = FileSystem, meta = (DisplayName = "Support iTunes File Sharing"))
	uint32 bSupportsITunesFileSharing : 1;
	
	// Whether files created by the app will be accessible from within the device's Files app (requires iTunes File Sharing)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = FileSystem, meta = (DisplayName = "Support Files App", EditCondition = "bSupportsITunesFileSharing"))
	uint32 bSupportsFilesApp : 1;
	
	// The Preferred Orientation will be used as the initial orientation at launch when both Landscape Left and Landscape Right orientations are to be supported.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = DeviceOrientations, meta = (DisplayName = "Preferred Landscape Orientation"))
	EIOSLandscapeOrientation PreferredLandscapeOrientation;

	// Specifies the the display name for the application. This will be displayed under the icon on the device.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = BundleInformation)
	FString BundleDisplayName;

	// Specifies the the name of the application bundle. This is the short name for the application bundle.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = BundleInformation)
	FString BundleName;

	// Specifies the bundle identifier for the application.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = BundleInformation)
	FString BundleIdentifier;

	// Specifies the version for the application.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = BundleInformation)
	FString VersionInfo;

	/**
	 * Choose whether to use a custom LaunchScreen.Storyboard as a Launchscreen. To use this option, create a storyboard in Xcode and 
	 * copy it named LaunchScreen.storyboard in Build/IOS/Resources/Interface under your Project folder. This will be compiled and 
	 * copied to the bundle app and the Launch screen image above will not be included in the app.
	 * When using assets in your custom LaunchScreen.storyboard, add them in Build/IOS/Resources/Interface/Assets and they will be included.
	 */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = LaunchScreen, meta = (DisplayName = "Custom Launchscreen Storyboard (experimental)", EditCondition = "!MacTargetPlatform.XcodeProjectSettings.ShouldDisableIOSSettings"))
    bool bCustomLaunchscreenStoryboard;

	// Whether the app supports Facebook
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Online)
	bool bEnableFacebookSupport;

	// Facebook App ID obtained from Facebook's Developer Centre
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Online, meta = (EditCondition = "bEnableFacebookSupport"))
	FString FacebookAppID;
    
    // Mobile provision to utilize when signing.
	// This value is stripped out when making builds.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build)
	FString MobileProvision;

	// Signing certificate to utilize when signing.
	// This value is stripped out when making builds.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build)
	FString SigningCertificate;
	
	// Whether to use automatic signing through Xcode
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (EditCondition = "!MacTargetPlatform.XcodeProjectSettings.ShouldDisableIOSSettings"))
	bool bAutomaticSigning;

	// The team ID of the apple developer account to be used to autmatically sign IOS builds.
	// This can be overridden in Turnkey with "RunUAT Turnkey -command=ManageSettings"
	// This value is stripped out when making builds.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (ConfigHierarchyEditable))
	FString IOSTeamID;

	// The username/email to use when logging in to DevCenter with Turnkey.
	// This can be overridden in Turnkey with "RunUAT Turnkey -command=ManageSettings"
	// This value is stripped out when making builds.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (ConfigHierarchyEditable))
	FString DevCenterUsername;
	
	// The password to use when logging in to DevCenter with Turnkey. NOTE: This is saved in plaintext, and is meant for shared accounts!
	// This value is stripped out when making builds.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Build, meta = (ConfigHierarchyEditable))
	FString DevCenterPassword;
	
	// Whether the app supports HTTPS
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Online, meta = (DisplayName = "Allow web connections to non-HTTPS websites"))
	bool bDisableHTTPS;


    // The Metal shader language version which will be used when compiling the shaders.
    UPROPERTY(EditAnywhere, config, Category=Rendering, meta = (DisplayName = "Metal Shader Standard To Target", ConfigRestartRequired = true))
	uint8 MetalLanguageVersion;
	
	/**
	 * Whether to use the Metal shading language's "fast" intrinsics.
	 * Fast intrinsics assume that no NaN or INF value will be provided as input,
	 * so are more efficient. However, they will produce undefined results if NaN/INF
	 * is present in the argument/s.
	 */
	UPROPERTY(EditAnywhere, config, Category=Rendering, meta = (DisplayName = "Use Fast-Math intrinsics", ConfigRestartRequired = true))
	bool UseFastIntrinsics;
	
	/**
	 * Whether to force Metal shaders to use 32bit floating point precision even when the shader uses half floats.
	 * Half floats are much more efficient when they are availble but have less accuracy over large ranges,
	 * as such some projects may need to use 32bit floats to ensure correct rendering.
	 */
	UPROPERTY(EditAnywhere, config, Category=Rendering, meta = (DisplayName = "Force 32bit Floating Point Precision", ConfigRestartRequired = true))
	bool ForceFloats;
	
	/**
	 * Whether to use of Metal shader-compiler's -ffast-math optimisations.
	 * Fast-Math performs algebraic-equivalent & reassociative optimisations not permitted by the floating point arithmetic standard (IEEE-754).
	 * These can improve shader performance at some cost to precision and can lead to NaN/INF propagation as they rely on
	 * shader inputs or variables not containing NaN/INF values. By default fast-math is enabled for performance.
	 */
	UPROPERTY(EditAnywhere, config, Category=Rendering, meta = (DisplayName = "Enable Fast-Math optimisations", ConfigRestartRequired = true))
	bool EnableMathOptimisations;
    
	// Whether or not the keyboard should be usable on it's own without a UITextField
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Input)
	bool bUseIntegratedKeyboard;

	// Whether or not we should receive the keyboard's visibility event (Shown/Hidden) regardless of the IntegratedKeyboard being activated or not.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = Input)
	bool bEnableVirtualKeyboardVisibilityEvent;
	
	// Whether or not the EditableText Widget is allowed to enable IntegratedKeyboard. bUseIntegratedKeyboard enabled will disregard this setting.
	UPROPERTY(GlobalConfig, EditAnywhere, Meta=(EditCondition = "!bUseIntegratedKeyboard"), Category = Input)
	bool bAllowWidgetEnablingIntegratedKeyboard;

	/** Sample rate to run the audio mixer with. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", Meta = (DisplayName = "Audio Mixer Sample Rate"))
	int32 AudioSampleRate;

	/** The amount of audio to compute each callback block. Lower values decrease latency but may increase CPU cost. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "512", ClampMax = "4096", DisplayName = "Callback Buffer Size"))
	int32 AudioCallbackBufferFrameSize;

	/** The number of buffers to keep enqueued. More buffers increases latency, but can compensate for variable compute availability in audio callbacks on some platforms. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "1", UIMin = "1", DisplayName = "Number of Buffers To Enqueue"))
	int32 AudioNumBuffersToEnqueue;

	/** The max number of channels (voices) to limit for this platform. The max channels used will be the minimum of this value and the global audio quality settings. A value of 0 will not apply a platform channel count max. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "0", UIMin = "0", DisplayName = "Max Channels"))
	int32 AudioMaxChannels;

	/** The number of workers to use to compute source audio. Will only use up to the max number of sources. Will evenly divide sources to each source worker. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "0", UIMin = "0", DisplayName = "Number of Source Workers"))
	int32 AudioNumSourceWorkers;

	/** Which of the currently enabled spatialization plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString SpatializationPlugin;

	/** Which of the currently enabled source data override plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString SourceDataOverridePlugin;

	/** Which of the currently enabled reverb plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString ReverbPlugin;

	/** Which of the currently enabled occlusion plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString OcclusionPlugin;

	/** Various overrides for how this platform should handle compression and decompression */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FPlatformRuntimeAudioCompressionOverrides CompressionOverrides;

    /** Whether this app's audio can be played when using other apps or on the springboard */
    UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (DisplayName = "Enable Background Audio"))
    bool bSupportsBackgroundAudio;

	/** This determines the max amount of memory that should be used for the cache at any given time. If set low (<= 8 MB), it lowers the size of individual chunks of audio during cook. */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides|Stream Caching", meta = (DisplayName = "Max Cache Size (KB)"))
	int32 CacheSizeKB;

	/** This overrides the default max chunk size used when chunking audio for stream caching (ignored if < 0) */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides|Stream Caching", meta = (DisplayName = "Max Chunk Size Override (KB)"))
	int32 MaxChunkSizeOverrideKB;

	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides")
	bool bResampleForDevice;

	/** Quality Level to COOK SoundCues at (if set, all other levels will be stripped by the cooker). */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides", meta = (DisplayName = "Sound Cue Cook Quality"))
	int32 SoundCueCookQualityIndex = INDEX_NONE;

	// Mapping of which sample rates are used for each sample rate quality for a specific platform.

	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "Max"))
	float MaxSampleRate;

	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "High"))
	float HighSampleRate;

	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "Medium"))
	float MedSampleRate;

	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "Low"))
	float LowSampleRate;

	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides|ResamplingQuality", meta = (DisplayName = "Min"))
	float MinSampleRate;

	// Scales all compression qualities when cooking to this platform. For example, 0.5 will halve all compression qualities, and 1.0 will leave them unchanged.
	UPROPERTY(config, EditAnywhere, Category = "Audio|CookOverrides")
	float CompressionQualityModifier;

	// When set to anything beyond 0, this will ensure any SoundWaves longer than this value, in seconds, to stream directly off of the disk.
	UPROPERTY(GlobalConfig)
	float AutoStreamingThreshold;

    virtual void PostReloadConfig(class FProperty* PropertyThatWasLoaded) override;

#if WITH_EDITOR
	// UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostInitProperties() override;
	// End of UObject interface
#endif

};
