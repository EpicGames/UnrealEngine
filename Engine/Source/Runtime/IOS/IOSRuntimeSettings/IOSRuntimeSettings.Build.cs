// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IOSRuntimeSettings : ModuleRules
{
	public IOSRuntimeSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "IOS";

        PublicDependencyModuleNames.Add("AudioPlatformConfiguration");

        PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine"
			}
		);
	}
}
