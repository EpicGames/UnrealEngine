// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSRuntimeSettings.h"
#include "HAL/FileManager.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/Paths.h"
#include "Misc/StringOutputDevice.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/ConfigCacheIni.h"

#if WITH_EDITOR
#include "Misc/App.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "DesktopPlatformModule.h"
#define LOCTEXT_NAMESPACE "IOSRuntimeSettings"
#endif

UIOSRuntimeSettings::UIOSRuntimeSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CacheSizeKB(65536)
	, MaxSampleRate(48000)
	, HighSampleRate(32000)
	, MedSampleRate(24000)
	, LowSampleRate(12000)
	, MinSampleRate(8000)
	, CompressionQualityModifier(1)
{
	bEnableStoreKitSupport = true;
	bEnableGameCenterSupport = true;
	bEnableCloudKitSupport = false;
	bUserSwitching = false;
	bSupportsPortraitOrientation = true;
	bSupportsITunesFileSharing = false;
	bSupportsFilesApp = false;
	BundleDisplayName = TEXT("UnrealGame");
	BundleName = TEXT("MyUnrealGame");
	BundleIdentifier = TEXT("com.YourCompany.GameNameNoSpaces");
	VersionInfo = TEXT("1.0.0");
    FrameRateLock = EPowerUsageFrameRateLock::PUFRL_30;
	bEnableDynamicMaxFPS = false;
	bSupportsIPad = true;
	bSupportsIPhone = true;
	bEnableSplitView = false;
	MinimumiOSVersion = EIOSVersion::IOS_17;
	bGeneratedSYMFile = false;
	bGeneratedSYMBundle = false;
	bGenerateXCArchive = false;
	bUsesNonExemptEncryption = false;
	ITSEncryptionExportComplianceCode = TEXT("");
	bSupportSecondaryMac = false;
	bUseRSync = true;
	bCustomLaunchscreenStoryboard = false;
	AdditionalPlistData = TEXT("");
	AdditionalLinkerFlags = TEXT("");
	AdditionalShippingLinkerFlags = TEXT("");
    bGameSupportsMultipleActiveControllers = false;
	bAllowRemoteRotation = true;
	bDisableMotionData = false;
    bEnableRemoteNotificationsSupport = false;
    bEnableBackgroundFetch = false;
	bSupportsMetal = true;
	bSupportsMetalMobileSM5 = false;
	bSupportsMetalMobileSM6 = false;
    bSupportHighRefreshRates = false;
	bDisableHTTPS = false;
    bSupportsBackgroundAudio = false;
	ScreenEdgesDeferredGestures = (int32)(EIOSScreenEdge::Top | EIOSScreenEdge::Left | EIOSScreenEdge::Bottom | EIOSScreenEdge::Right);
}

void UIOSRuntimeSettings::PostReloadConfig(class FProperty* PropertyThatWasLoaded)
{
	Super::PostReloadConfig(PropertyThatWasLoaded);

#if PLATFORM_IOS

	FPlatformApplicationMisc::SetGamepadsAllowed(bAllowControllers);

#endif //PLATFORM_IOS
}

#if WITH_EDITOR
void UIOSRuntimeSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Ensure that at least one orientation is supported
	if (!bSupportsPortraitOrientation && !bSupportsUpsideDownOrientation && !bSupportsLandscapeLeftOrientation && !bSupportsLandscapeRightOrientation)
	{
		bSupportsPortraitOrientation = true;
		UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, bSupportsPortraitOrientation)), GetDefaultConfigFilename());
	}

	// Ensure that at least one API is supported
	if (!bSupportsMetal && !bSupportsMetalMobileSM5 && !bSupportsMetalMobileSM6)
	{
		bSupportsMetal = true;
		UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, bSupportsMetal)), GetDefaultConfigFilename());
	}

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, MinimumiOSVersion)
		&& (MinimumiOSVersion == EIOSVersion::IOS_15 
			|| MinimumiOSVersion == EIOSVersion::IOS_16))
	{
		FMessageDialog::Open(
			EAppMsgCategory::Warning,
			EAppMsgType::Ok,
			LOCTEXT("MinimumIOSVersionChanged_ShaderStrippingReminder",
				"You set the Minimum iOS Version to iOS 15 or iOS 16.\n\n"
				"Remember to disable Metal Shader Stripping (r.Shaders.Symbols=1) to avoid the 'Corrupted library' "
				"assertion at PSO creation caused by the Xcode 26 / 16.4 metal-strip toolchain."),
			LOCTEXT("MinimumIOSVersionChanged_Title", "Minimum iOS Version Changed"));
	}
}


void UIOSRuntimeSettings::PostInitProperties()
{
	Super::PostInitProperties();

	// Migrate old values using MRT to SM5
	bool bSupportsMetalMRT = false;
	if (GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetalMRT"), bSupportsMetalMRT, GEngineIni))
	{
		bSupportsMetalMobileSM5 = bSupportsMetalMRT;
		UpdateSinglePropertyInConfigFile(
			GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, bSupportsMetalMobileSM5)),
			GetDefaultConfigFilename());
		GConfig->RemoveKey(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetalMRT"), GEngineIni);
	}
	
	// We can have a look for potential keys
	if (!RemoteServerName.IsEmpty() && !RSyncUsername.IsEmpty())
	{
		SSHPrivateKeyLocation = TEXT("");

		FString RealRemoteServerName = RemoteServerName;
		if(RemoteServerName.Contains(TEXT(":")))
		{
			FString RemoteServerPort;
			RemoteServerName.Split(TEXT(":"),&RealRemoteServerName,&RemoteServerPort);
		}
		const FString DefaultKeyFilename = "RemoteToolChainPrivate.key";
		const FString RelativeFilePathLocation = FPaths::Combine(TEXT("SSHKeys"), *RealRemoteServerName, *RSyncUsername, *DefaultKeyFilename);

		FString Path = FPlatformMisc::GetEnvironmentVariable(TEXT("APPDATA"));

		TArray<FString> PossibleKeyLocations;
		PossibleKeyLocations.Add(FPaths::Combine(*FPaths::ProjectDir(), TEXT("Restricted"), TEXT("NotForLicensees"), TEXT("Build"), *RelativeFilePathLocation));
		PossibleKeyLocations.Add(FPaths::Combine(*FPaths::ProjectDir(), TEXT("Restricted"), TEXT("NoRedist"), TEXT("Build"), *RelativeFilePathLocation));
		PossibleKeyLocations.Add(FPaths::Combine(*FPaths::ProjectDir(), TEXT("Build"), *RelativeFilePathLocation));
		PossibleKeyLocations.Add(FPaths::Combine(*FPaths::EngineDir(), TEXT("Restricted"), TEXT("NotForLicensees"), TEXT("Build"), TEXT("NotForLicensees"), *RelativeFilePathLocation));
		PossibleKeyLocations.Add(FPaths::Combine(*FPaths::EngineDir(), TEXT("Restricted"), TEXT("NoRedist"), TEXT("Build"), *RelativeFilePathLocation));
		PossibleKeyLocations.Add(FPaths::Combine(*FPaths::EngineDir(), TEXT("Build"), *RelativeFilePathLocation));
		PossibleKeyLocations.Add(FPaths::Combine(*Path, TEXT("Unreal Engine"), TEXT("UnrealBuildTool"), *RelativeFilePathLocation));

		// Find a potential path that we will use if the user hasn't overridden.
		// For information purposes only
		for (const FString& NextLocation : PossibleKeyLocations)
		{
			if (IFileManager::Get().FileSize(*NextLocation) > 0)
			{
				SSHPrivateKeyLocation = NextLocation;
				break;
			}
		}
	}

	if ((MinimumiOSVersion < EIOSVersion::IOS_15) && (MinimumiOSVersion != EIOSVersion::IOS_Minimum))
	{
		MinimumiOSVersion = EIOSVersion::IOS_Minimum;
		UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, MinimumiOSVersion)), GetDefaultConfigFilename());
	}
	if (!bSupportsMetal && !bSupportsMetalMobileSM5 && !bSupportsMetalMobileSM6)
	{
		bSupportsMetal = true;
		UpdateSinglePropertyInConfigFile(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UIOSRuntimeSettings, bSupportsMetal)), GetDefaultConfigFilename());
	}
}

#undef LOCTEXT_NAMESPACE

#endif
