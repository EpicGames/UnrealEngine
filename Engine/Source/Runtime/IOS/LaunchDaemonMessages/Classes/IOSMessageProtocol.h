// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "UObject/ObjectMacros.h"
#include "IOSMessageProtocol.generated.h"


USTRUCT()
struct FIOSLaunchDaemonPing
{
	GENERATED_USTRUCT_BODY()

	//FIOSLaunchDaemonPing() {}
};


USTRUCT()
struct FIOSLaunchDaemonPong
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString DeviceID;

	UPROPERTY()
	FString DeviceUDID;

	UPROPERTY()
	FString DeviceName;

	UPROPERTY()
	FString DeviceStatus;

	UPROPERTY()
	FString DeviceType;

	UPROPERTY()
	FString DeviceOSVersion;

	UPROPERTY()
	FString DeviceModelId;

	UPROPERTY()
	FString DeviceConnectionType;

	UPROPERTY()
	bool bCanPowerOff;

	UPROPERTY()
	bool bCanPowerOn;

	UPROPERTY()
	bool bIsAuthorized;

	UPROPERTY()
	bool bCanReboot;

	FIOSLaunchDaemonPong()
		: bCanPowerOff(false)
		, bCanPowerOn(false)
		, bIsAuthorized(false)
		, bCanReboot(false)
	{}

	FIOSLaunchDaemonPong(FString InDeviceID, FString InDeviceUDID, FString InDeviceName, FString InDeviceStatus, FString InDeviceType, FString InDeviceConnectionType, bool bInCanPowerOff, bool bInCanPowerOn, bool bInCanReboot, bool bIsAuthorized)
		: DeviceID(InDeviceID)
		, DeviceUDID(InDeviceUDID)
		, DeviceName(InDeviceName)
		, DeviceStatus(InDeviceStatus)
		, DeviceType(InDeviceType)
		, DeviceConnectionType(InDeviceConnectionType)
		, bCanPowerOff(bInCanPowerOff)
		, bCanPowerOn(bInCanPowerOn)
		, bIsAuthorized(bIsAuthorized)
		, bCanReboot(bInCanReboot)
	{}
};


USTRUCT()
struct FIOSLaunchDaemonLaunchApp
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString AppID;

	UPROPERTY()
	FString Parameters;

	FIOSLaunchDaemonLaunchApp() {}

	FIOSLaunchDaemonLaunchApp(FString InAppID, FString Params)
		: AppID(InAppID)
		, Parameters(Params)
	{}
};
