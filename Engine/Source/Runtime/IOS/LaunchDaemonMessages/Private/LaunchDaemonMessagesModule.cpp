// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


/**
 * Implements the EngineMessages module.
 */
class FLaunchDaemonMessagesModule
	: public IModuleInterface
{
public:

	virtual void StartupModule( ) override { }

	virtual void ShutdownModule( ) override { }

	virtual bool SupportsDynamicReloading( ) override
	{
		return true;
	}
};


IMPLEMENT_MODULE(FLaunchDaemonMessagesModule, LaunchDaemonMessages);
