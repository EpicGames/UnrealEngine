// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;

public class MarketplaceKit : ModuleRules
{
	public MarketplaceKit(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"Engine",
		});

		// Embed the pre-built MarketplaceKitWrapper framework into the app bundle
		PublicAdditionalFrameworks.Add(new Framework(
			"MarketplaceKitWrapper",
			Path.Combine(ModuleDirectory, "MarketplaceKitWrapper.embeddedframework.zip"),
			Framework.FrameworkMode.Copy));

		// Weak-link so the app launches even if the framework is unavailable at runtime
		PublicWeakFrameworks.Add("MarketplaceKitWrapper");
	}
}
