// Copyright Epic Games, Inc. All Rights Reserved.

#include "MarketplaceKit.h"

#include "CoreGlobals.h"
#include "MarketplaceKitWrapperAPI.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Logging/LogMacros.h"

#include <dispatch/dispatch.h>

DEFINE_LOG_CATEGORY_STATIC(LogMarketplaceKit, Log, All);

static TAutoConsoleVariable<bool> CVarForceEmptyEligibilityRegion(
	TEXT("MarketplaceKit.ForceEmptyEligibilityRegion"),
	false,
	TEXT("Force GetEligibilityRegionStatic to return an empty string, simulating a device with no eligibility region.")
);

static FMarketplaceKitModule* GPendingEligibilityRegionSelf = nullptr;

static bool IsMarketplaceKitWrapperAvailable()
{
	// Weak-linked symbols are nullptr if the framework isn't loaded
	return MKW_GetCurrent != nullptr;
}

const FString LexToString(EMarketplaceType Value)
{
	switch (Value)
	{
		case EMarketplaceType::AppStore: return "AppStore";
		case EMarketplaceType::TestFlight: return "TestFlight";
		case EMarketplaceType::Marketplace: return "Marketplace";
		case EMarketplaceType::Web: return "Web";
		case EMarketplaceType::Other: return "Other";
		default: checkNoEntry(); [[fallthrough]];
		case EMarketplaceType::NotAvailable: return "NotAvailable";
	}
}

void LexFromString(EMarketplaceType& OutValue, const TCHAR* InValue)
{
	if (FCString::Stricmp(InValue, TEXT("AppStore")) == 0)
	{
		OutValue = EMarketplaceType::AppStore;
	}
	else if (FCString::Stricmp(InValue, TEXT("TestFlight")) == 0)
	{
		OutValue = EMarketplaceType::TestFlight;
	}
	else if (FCString::Stricmp(InValue, TEXT("Marketplace")) == 0)
	{
		OutValue = EMarketplaceType::Marketplace;
	}
	else if (FCString::Stricmp(InValue, TEXT("Web")) == 0)
	{
		OutValue = EMarketplaceType::Web;
	}
	else if (FCString::Stricmp(InValue, TEXT("Other")) == 0)
	{
		OutValue = EMarketplaceType::Other;
	}
	else if (FCString::Stricmp(InValue, TEXT("NotAvailable")) == 0)
	{
		OutValue = EMarketplaceType::NotAvailable;
	}
	else
	{
		checkNoEntry();
		OutValue = EMarketplaceType::NotAvailable;
	}
}

void FMarketplaceKitModule::StartupModule()
{
	FString TestFlightMarketplaceTypeString;
	GConfig->GetString(TEXT("MarketplaceKit"), TEXT("TestFlightMarketplaceType"), TestFlightMarketplaceTypeString, GEngineIni);
	FParse::Value(FCommandLine::Get(), TEXT("TestFlightMarketplaceType="), TestFlightMarketplaceTypeString);
	if (!TestFlightMarketplaceTypeString.IsEmpty())
	{
		LexFromString(TestFlightMarketplaceType, *TestFlightMarketplaceTypeString);
	}

	GConfig->GetString(TEXT("MarketplaceKit"), TEXT("TestFlightMarketplaceBundleId"), TestFlightMarketplaceBundleId, GEngineIni);
	FParse::Value(FCommandLine::Get(), TEXT("TestFlightMarketplaceBundleId="), TestFlightMarketplaceBundleId);
	
	GConfig->GetString(TEXT("MarketplaceKit"), TEXT("TestOverrideEligibilityRegion"), TestOverrideEligibilityRegion, GEngineIni);
	FParse::Value(FCommandLine::Get(), TEXT("TestOverrideEligibilityRegion="), TestOverrideEligibilityRegion);
	
	if (!IsMarketplaceKitWrapperAvailable())
	{
		UE_LOGF(LogMarketplaceKit, Warning, "MarketplaceKitWrapper framework not available, marketplace features will be disabled");
	}

	CacheValue();
	GetEligibilityRegionAsync();
}

void FMarketplaceKitModule::ShutdownModule()
{
	GPendingEligibilityRegionSelf = nullptr;
}

bool FMarketplaceKitModule::SupportsDynamicReloading()
{
	return true;
}

static constexpr EMarketplaceType ConvertMarketplaceType(const long Type)
{
	switch (Type)
	{
		case 0:		return EMarketplaceType::AppStore;
		case 1:		return EMarketplaceType::TestFlight;
		case 2:		return EMarketplaceType::Marketplace;
		case 3:		return EMarketplaceType::Web;
		case 4:		return EMarketplaceType::Other;
		case 5:		return EMarketplaceType::AppStore; // NotAvailable: Pre 17.4, hardcode to AppStore
		default:								return EMarketplaceType::Other;
	}
}

void FMarketplaceKitModule::GetCurrentTypeAsync(TFunction<void(EMarketplaceType Type, const FString& Name)> Callback)
{
	if (!IsMarketplaceKitWrapperAvailable())
	{
		Callback(EMarketplaceType::NotAvailable, FString());
		return;
	}

	// C callbacks don't support captures, so store context in statics.
	// The Swift side dispatches the callback on the main thread, so only one can be in flight at a time.
	static TFunction<void(EMarketplaceType, const FString&)>* PendingGetCurrentCallback = nullptr;
	static FMarketplaceKitModule* PendingSelf = nullptr;

	PendingGetCurrentCallback = new TFunction<void(EMarketplaceType, const FString&)>(MoveTemp(Callback));
	PendingSelf = this;

	MKW_GetCurrent([](long Type, const char* Name)
		{
			const EMarketplaceType ConvertedType = ConvertMarketplaceType(Type);
		const FString ConvertedName = FString(UTF8_TO_TCHAR(Name ? Name : ""));

		UE_LOGF(LogMarketplaceKit, Log, "MKW_GetCurrent callback %ls %ls", *LexToString(ConvertedType), *ConvertedName);

		if (PendingSelf)
		{
			PendingSelf->CachedType = ConvertedType;
			PendingSelf->CachedName = ConvertedName;
			PendingSelf->bCachedTypeValid = true;
		}

		if (PendingGetCurrentCallback)
		{
			EMarketplaceType EffectiveType;
			FString EffectiveName;
			if (PendingSelf)
			{
				PendingSelf->GetEffectiveType(EffectiveType, EffectiveName);
			}
			else
			{
				EffectiveType = ConvertedType;
				EffectiveName = ConvertedName;
			}

			(*PendingGetCurrentCallback)(EffectiveType, EffectiveName);
			delete PendingGetCurrentCallback;
			PendingGetCurrentCallback = nullptr;
			PendingSelf = nullptr;
		}
	});
}

void FMarketplaceKitModule::RequestCTTokenAsync(TDelegate<void(bool Success, FString Output)> Callback)
{
	if (!MKW_RequestCTToken)
	{
		Callback.ExecuteIfBound(false, FString(TEXT("MarketplaceKitWrapper framework not loaded")));
		return;
	}

	static TDelegate<void(bool, FString)> PendingCTTokenCallback;
	PendingCTTokenCallback = MoveTemp(Callback);

	MKW_RequestCTToken([](bool Success, const char* Output)
	{
		PendingCTTokenCallback.ExecuteIfBound(Success, FString(UTF8_TO_TCHAR(Output ? Output : "")));
		PendingCTTokenCallback.Unbind();
	});
}

void FMarketplaceKitModule::GetEligibilityRegionAsync()
{
	// Check the symbol directly rather than IsMarketplaceKitWrapperAvailable() since that checks
	// MKW_GetCurrent. If this symbol is unavailable, bCachedEligibilityRegionInitialized will
	// never be set and GetEligibilityRegionStatic will silently return "".
	if (!MKW_GetEligibilityRegion)
	{
		UE_LOGF(LogMarketplaceKit, Warning, "CTC Reporting: Cannot initialize eligibility region.");
		return;
	}
	
	UE_LOGF(LogMarketplaceKit, Verbose, "CTC Reporting: Querying eligibility region");

	// File-scope static (rather than function-local like other callbacks in this file) so that
	// ShutdownModule can null it out if the module is unloaded before the callback fires.
	GPendingEligibilityRegionSelf = this;

	MKW_GetEligibilityRegion([](bool Success, const char* Output)
	{
		const FString OutputString(UTF8_TO_TCHAR(Output ? Output : ""));

		if (GPendingEligibilityRegionSelf)
		{
			if (Success)
			{
				GPendingEligibilityRegionSelf->CachedEligibilityRegion = OutputString;
			}

			// Store with default seq_cst ordering ensures CachedEligibilityRegion is visible to
			// any thread that subsequently loads bCachedEligibilityRegionInitialized and sees true.
			GPendingEligibilityRegionSelf->bCachedEligibilityRegionInitialized.Store(true);
			GPendingEligibilityRegionSelf = nullptr;

			UE_LOGF(LogMarketplaceKit, Verbose, "CTC Reporting: [%ls] Obtained eligibility region [%ls]",
					Success ? TEXT("SUCCESS") : TEXT("FAILURE"), *OutputString);
		}
	});
}

FString FMarketplaceKitModule::GetEligibilityRegionStatic()
{
	if (CVarForceEmptyEligibilityRegion.GetValueOnAnyThread())
	{
		UE_LOGF(LogMarketplaceKit, Verbose, "CTC Reporting: Force empty eligibility region is set, returning empty string.");
		return FString();
	}

	if (FMarketplaceKitModule* Module = FModuleManager::LoadModulePtr<FMarketplaceKitModule>(TEXT("MarketplaceKit")))
	{
		if (!Module->TestOverrideEligibilityRegion.IsEmpty())
		{
			UE_LOGF(LogMarketplaceKit, Verbose, "CTC Reporting: Using overridden eligibility region [%ls]", *Module->TestOverrideEligibilityRegion);
			return Module->TestOverrideEligibilityRegion;
		}
		
		if (Module->bCachedEligibilityRegionInitialized.Load())
		{
			return Module->CachedEligibilityRegion;
		}
	}
	return FString();
}

void FMarketplaceKitModule::GetCurrentType(EMarketplaceType& OutType, FString& OutName)
{
	CacheValue();
	GetEffectiveType(OutType, OutName);
}

EMarketplaceType FMarketplaceKitModule::GetCurrentTypeStatic()
{
	EMarketplaceType Result = EMarketplaceType::NotAvailable;
	if (FMarketplaceKitModule* MarketplaceKitModule = FModuleManager::LoadModulePtr<FMarketplaceKitModule>(TEXT("MarketplaceKit")))
	{
		FString Unused;
		MarketplaceKitModule->GetCurrentType(Result, Unused);
	}
	return Result;
}

FString FMarketplaceKitModule::GetCurrentTypeAsString()
{
	CacheValue();
	
	TStringBuilder<256> Result;

	EMarketplaceType EffectiveType;
	FString EffectiveName;
	GetEffectiveType(EffectiveType, EffectiveName);

	switch (EffectiveType)
	{
		case EMarketplaceType::AppStore:		Result.Append(TEXT("AppStore")); break;
		case EMarketplaceType::TestFlight:		Result.Append(TEXT("TestFlight")); break;
		case EMarketplaceType::Marketplace:		Result.Append(TEXT("Marketplace")); break;
		case EMarketplaceType::Web:				Result.Append(TEXT("Web")); break;
		case EMarketplaceType::NotAvailable:	Result.Append(TEXT("NotAvailable")); break;
		case EMarketplaceType::Other:			[[fallthrough]];
		default:								Result.Append(TEXT("Other")); break;
	}
	
	if (EffectiveType == EMarketplaceType::Marketplace && !EffectiveName.IsEmpty())
	{
		Result.Append(TEXT("-"));
		Result.Append(MoveTemp(EffectiveName));
	}

	return *Result;
}

FString FMarketplaceKitModule::GetCurrentTypeAsStringStatic()
{
	if (FMarketplaceKitModule* MarketplaceKitModule = FModuleManager::LoadModulePtr<FMarketplaceKitModule>(TEXT("MarketplaceKit")))
	{
		return MarketplaceKitModule->GetCurrentTypeAsString();
	}
	return FString();
}

FString FMarketplaceKitModule::GetMarketplaceBundleIdStatic()
{
	if (FMarketplaceKitModule* MarketplaceKitModule = FModuleManager::LoadModulePtr<FMarketplaceKitModule>(TEXT("MarketplaceKit")))
	{
		EMarketplaceType Type;
		FString Name;
		MarketplaceKitModule->GetCurrentType(Type, Name);
		if (Type == EMarketplaceType::Marketplace)
		{
			return Name;
		}
	}
	return FString();
}

bool FMarketplaceKitModule::IsTestFlight()
{
	CacheValue();
	return CachedType == EMarketplaceType::TestFlight;
}

bool FMarketplaceKitModule::IsTestFlightStatic()
{
	if (FMarketplaceKitModule* MarketplaceKitModule = FModuleManager::LoadModulePtr<FMarketplaceKitModule>(TEXT("MarketplaceKit")))
	{
		return MarketplaceKitModule->IsTestFlight();
	}
	return false;
}

void FMarketplaceKitModule::CacheValue()
{
	if (bCachedTypeValid)
	{
		return;
	}

	if (!IsMarketplaceKitWrapperAvailable())
	{
		return;
	}

	// TODO avoid scheduling multiple requests in case this path is hit from multiple threads

	dispatch_semaphore_t Semaphore = dispatch_semaphore_create(0);

	static EMarketplaceType* PendingCacheType = nullptr;
	static FString* PendingCacheName = nullptr;
	static bool* PendingCacheValid = nullptr;
	static dispatch_semaphore_t PendingCacheSemaphore = nullptr;

	PendingCacheType = &CachedType;
	PendingCacheName = &CachedName;
	PendingCacheValid = &bCachedTypeValid;
	PendingCacheSemaphore = Semaphore;

	MKW_GetCurrent([](long Type, const char* Name)
	{
		*PendingCacheType = ConvertMarketplaceType(Type);
		*PendingCacheName = FString(UTF8_TO_TCHAR(Name ? Name : ""));
		*PendingCacheValid = true;

		UE_LOGF(LogMarketplaceKit, Log, "MKW_GetCurrent CacheValue callback %ls %ls", *LexToString(*PendingCacheType), **PendingCacheName);

		dispatch_semaphore_signal(PendingCacheSemaphore);
	});

	// wait for a result, but timeout after 1s
	dispatch_semaphore_wait(Semaphore, DISPATCH_TIME_FOREVER);
    dispatch_release(Semaphore);
}

void FMarketplaceKitModule::GetEffectiveType(EMarketplaceType& OutType, FString& OutName)
{
	if (CachedType == EMarketplaceType::TestFlight
		// Other == run from XCode
		|| CachedType == EMarketplaceType::Other)
	{
		OutType = TestFlightMarketplaceType;
		OutName = TestFlightMarketplaceBundleId;
		return;
	}
	OutType = CachedType;
	OutName = CachedName;
}

IMPLEMENT_MODULE(FMarketplaceKitModule, MarketplaceKit);
