// Copyright Epic Games, Inc. All Rights Reserved.
// Generated by Apple Swift version 6.3 effective-5.10 (swiftlang-6.3.0.123.5 clang-2100.0.123.102)
#ifndef MARKETPLACEKITWRAPPER_SWIFT_H
#define MARKETPLACEKITWRAPPER_SWIFT_H
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wgcc-compat"

#if !defined(__has_include)
# define __has_include(x) 0
#endif
#if !defined(__has_attribute)
# define __has_attribute(x) 0
#endif
#if !defined(__has_feature)
# define __has_feature(x) 0
#endif
#if !defined(__has_warning)
# define __has_warning(x) 0
#endif

#if __has_include(<swift/objc-prologue.h>)
# include <swift/objc-prologue.h>
#endif

#pragma clang diagnostic ignored "-Wauto-import"
#if defined(__OBJC__)
#include <Foundation/Foundation.h>
#endif // defined(__OBJC__)
#if defined(__cplusplus)
#include <cstdint>
#include <cstddef>
#include <cstdbool>
#include <cstring>
#include <stdlib.h>
#include <new>
#include <type_traits>
#else
#include <stdint.h>
#include <stddef.h>
#include <stdbool.h>
#include <string.h>
#endif
#if defined(__cplusplus)
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wnon-modular-include-in-framework-module"
#if defined(__arm64e__) && __has_include(<ptrauth.h>)
# include <ptrauth.h>
#else
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wreserved-macro-identifier"
# ifndef __ptrauth_swift_value_witness_function_pointer
#  define __ptrauth_swift_value_witness_function_pointer(x)
# endif
# ifndef __ptrauth_swift_class_method_pointer
#  define __ptrauth_swift_class_method_pointer(x)
# endif
#pragma clang diagnostic pop
#endif
#pragma clang diagnostic pop
#endif

#if !defined(SWIFT_TYPEDEFS)
# define SWIFT_TYPEDEFS 1
# if __has_include(<uchar.h>)
#  include <uchar.h>
# elif !defined(__cplusplus)
typedef unsigned char char8_t;
typedef uint_least16_t char16_t;
typedef uint_least32_t char32_t;
# endif
typedef float swift_float2  __attribute__((__ext_vector_type__(2)));
typedef float swift_float3  __attribute__((__ext_vector_type__(3)));
typedef float swift_float4  __attribute__((__ext_vector_type__(4)));
typedef double swift_double2  __attribute__((__ext_vector_type__(2)));
typedef double swift_double3  __attribute__((__ext_vector_type__(3)));
typedef double swift_double4  __attribute__((__ext_vector_type__(4)));
typedef int swift_int2  __attribute__((__ext_vector_type__(2)));
typedef int swift_int3  __attribute__((__ext_vector_type__(3)));
typedef int swift_int4  __attribute__((__ext_vector_type__(4)));
typedef unsigned int swift_uint2  __attribute__((__ext_vector_type__(2)));
typedef unsigned int swift_uint3  __attribute__((__ext_vector_type__(3)));
typedef unsigned int swift_uint4  __attribute__((__ext_vector_type__(4)));
#endif

#if !defined(SWIFT_PASTE)
# define SWIFT_PASTE_HELPER(x, y) x##y
# define SWIFT_PASTE(x, y) SWIFT_PASTE_HELPER(x, y)
#endif
#if !defined(SWIFT_METATYPE)
# define SWIFT_METATYPE(X) Class
#endif
#if !defined(SWIFT_CLASS_PROPERTY)
# if __has_feature(objc_class_property)
#  define SWIFT_CLASS_PROPERTY(...) __VA_ARGS__
# else
#  define SWIFT_CLASS_PROPERTY(...) 
# endif
#endif
#if !defined(SWIFT_RUNTIME_NAME)
# if __has_attribute(objc_runtime_name)
#  define SWIFT_RUNTIME_NAME(X) __attribute__((objc_runtime_name(X)))
# else
#  define SWIFT_RUNTIME_NAME(X) 
# endif
#endif
#if !defined(SWIFT_COMPILE_NAME)
# if __has_attribute(swift_name)
#  define SWIFT_COMPILE_NAME(X) __attribute__((swift_name(X)))
# else
#  define SWIFT_COMPILE_NAME(X) 
# endif
#endif
#if !defined(SWIFT_METHOD_FAMILY)
# if __has_attribute(objc_method_family)
#  define SWIFT_METHOD_FAMILY(X) __attribute__((objc_method_family(X)))
# else
#  define SWIFT_METHOD_FAMILY(X) 
# endif
#endif
#if !defined(SWIFT_NOESCAPE)
# if __has_attribute(noescape)
#  define SWIFT_NOESCAPE __attribute__((noescape))
# else
#  define SWIFT_NOESCAPE 
# endif
#endif
#if !defined(SWIFT_RELEASES_ARGUMENT)
# if __has_attribute(ns_consumed)
#  define SWIFT_RELEASES_ARGUMENT __attribute__((ns_consumed))
# else
#  define SWIFT_RELEASES_ARGUMENT 
# endif
#endif
#if !defined(SWIFT_WARN_UNUSED_RESULT)
# if __has_attribute(warn_unused_result)
#  define SWIFT_WARN_UNUSED_RESULT __attribute__((warn_unused_result))
# else
#  define SWIFT_WARN_UNUSED_RESULT 
# endif
#endif
#if !defined(SWIFT_NORETURN)
# if __has_attribute(noreturn)
#  define SWIFT_NORETURN __attribute__((noreturn))
# else
#  define SWIFT_NORETURN 
# endif
#endif
#if !defined(SWIFT_CLASS_EXTRA)
# define SWIFT_CLASS_EXTRA 
#endif
#if !defined(SWIFT_PROTOCOL_EXTRA)
# define SWIFT_PROTOCOL_EXTRA 
#endif
#if !defined(SWIFT_ENUM_EXTRA)
# define SWIFT_ENUM_EXTRA 
#endif
#if !defined(SWIFT_CLASS)
# if __has_attribute(objc_subclassing_restricted)
#  define SWIFT_CLASS(SWIFT_NAME) SWIFT_RUNTIME_NAME(SWIFT_NAME) __attribute__((objc_subclassing_restricted)) SWIFT_CLASS_EXTRA
#  define SWIFT_CLASS_NAMED(SWIFT_NAME) __attribute__((objc_subclassing_restricted)) SWIFT_COMPILE_NAME(SWIFT_NAME) SWIFT_CLASS_EXTRA
# else
#  define SWIFT_CLASS(SWIFT_NAME) SWIFT_RUNTIME_NAME(SWIFT_NAME) SWIFT_CLASS_EXTRA
#  define SWIFT_CLASS_NAMED(SWIFT_NAME) SWIFT_COMPILE_NAME(SWIFT_NAME) SWIFT_CLASS_EXTRA
# endif
#endif
#if !defined(SWIFT_RESILIENT_CLASS)
# if __has_attribute(objc_class_stub)
#  define SWIFT_RESILIENT_CLASS(SWIFT_NAME) SWIFT_CLASS(SWIFT_NAME) __attribute__((objc_class_stub))
#  define SWIFT_RESILIENT_CLASS_NAMED(SWIFT_NAME) __attribute__((objc_class_stub)) SWIFT_CLASS_NAMED(SWIFT_NAME)
# else
#  define SWIFT_RESILIENT_CLASS(SWIFT_NAME) SWIFT_CLASS(SWIFT_NAME)
#  define SWIFT_RESILIENT_CLASS_NAMED(SWIFT_NAME) SWIFT_CLASS_NAMED(SWIFT_NAME)
# endif
#endif
#if !defined(SWIFT_PROTOCOL)
# define SWIFT_PROTOCOL(SWIFT_NAME) SWIFT_RUNTIME_NAME(SWIFT_NAME) SWIFT_PROTOCOL_EXTRA
# define SWIFT_PROTOCOL_NAMED(SWIFT_NAME) SWIFT_COMPILE_NAME(SWIFT_NAME) SWIFT_PROTOCOL_EXTRA
#endif
#if !defined(SWIFT_EXTENSION)
# define SWIFT_EXTENSION(M) SWIFT_PASTE(M##_Swift_, __LINE__)
#endif
#if !defined(OBJC_DESIGNATED_INITIALIZER)
# if __has_attribute(objc_designated_initializer)
#  define OBJC_DESIGNATED_INITIALIZER __attribute__((objc_designated_initializer))
# else
#  define OBJC_DESIGNATED_INITIALIZER 
# endif
#endif
#if !defined(SWIFT_ENUM_ATTR)
# if __has_attribute(enum_extensibility)
#  define SWIFT_ENUM_ATTR(_extensibility) __attribute__((enum_extensibility(_extensibility)))
# else
#  define SWIFT_ENUM_ATTR(_extensibility) 
# endif
#endif
#if !defined(SWIFT_ENUM)
# if (defined(__cplusplus) && __cplusplus >= 201103L) ||      (defined(__STDC_VERSION__) && __STDC_VERSION__ >= 202311L) ||      __has_feature(objc_fixed_enum)
#  define SWIFT_ENUM(_type, _name, _extensibility) enum _name : _type _name; enum SWIFT_ENUM_ATTR(_extensibility) SWIFT_ENUM_EXTRA _name : _type
#  if __has_feature(generalized_swift_name)
#   define SWIFT_ENUM_NAMED(_type, _name, SWIFT_NAME, _extensibility) enum _name : _type _name SWIFT_COMPILE_NAME(SWIFT_NAME); enum SWIFT_COMPILE_NAME(SWIFT_NAME) SWIFT_ENUM_ATTR(_extensibility) SWIFT_ENUM_EXTRA _name : _type
#  else
#   define SWIFT_ENUM_NAMED(_type, _name, SWIFT_NAME, _extensibility) SWIFT_ENUM(_type, _name, _extensibility)
#  endif
# else
#  define SWIFT_ENUM(_type, _name, _extensibility) _type _name; enum 
#  define SWIFT_ENUM_NAMED(_type, _name, SWIFT_NAME, _extensibility) SWIFT_ENUM(_type, _name, _extensibility)
# endif
#endif
#if !defined(SWIFT_ENUM_TAG)
# if (defined(__cplusplus) && __cplusplus >= 201103L) ||      (defined(__STDC_VERSION__) && __STDC_VERSION__ >= 202311L) ||      __has_feature(objc_fixed_enum)
#  define SWIFT_ENUM_TAG enum
# else
#  define SWIFT_ENUM_TAG
# endif
#endif
#if !defined(SWIFT_ENUM_FWD_DECL)
# if (defined(__cplusplus) && __cplusplus >= 201103L) ||      (defined(__STDC_VERSION__) && __STDC_VERSION__ >= 202311L) ||      __has_feature(objc_fixed_enum)
#  define SWIFT_ENUM_FWD_DECL(_type, _name) enum _name : _type;
# else
#  define SWIFT_ENUM_FWD_DECL(_type, _name) typedef _type _name;
# endif
#endif
#if !defined(SWIFT_UNAVAILABLE)
# define SWIFT_UNAVAILABLE __attribute__((unavailable))
#endif
#if !defined(SWIFT_UNAVAILABLE_MSG)
# define SWIFT_UNAVAILABLE_MSG(msg) __attribute__((unavailable(msg)))
#endif
#if !defined(SWIFT_AVAILABILITY)
# define SWIFT_AVAILABILITY(plat, ...) __attribute__((availability(plat, __VA_ARGS__)))
#endif
#if !defined(SWIFT_AVAILABILITY_DOMAIN)
# define SWIFT_AVAILABILITY_DOMAIN(dom, ...) __attribute__((availability(domain: dom, __VA_ARGS__)))
#endif
#if !defined(SWIFT_WEAK_IMPORT)
# define SWIFT_WEAK_IMPORT __attribute__((weak_import))
#endif
#if !defined(SWIFT_DEPRECATED)
# define SWIFT_DEPRECATED __attribute__((deprecated))
#endif
#if !defined(SWIFT_DEPRECATED_MSG)
# define SWIFT_DEPRECATED_MSG(...) __attribute__((deprecated(__VA_ARGS__)))
#endif
#if !defined(SWIFT_DEPRECATED_OBJC)
# if __has_feature(attribute_diagnose_if_objc)
#  define SWIFT_DEPRECATED_OBJC(Msg) __attribute__((diagnose_if(1, Msg, "warning")))
# else
#  define SWIFT_DEPRECATED_OBJC(Msg) SWIFT_DEPRECATED_MSG(Msg)
# endif
#endif
#if defined(__OBJC__)
#if !defined(IBSegueAction)
# define IBSegueAction 
#endif
#endif
#if !defined(SWIFT_EXTERN)
# if defined(__cplusplus)
#  define SWIFT_EXTERN extern "C"
# else
#  define SWIFT_EXTERN extern
# endif
#endif
#if !defined(SWIFT_CALL)
# define SWIFT_CALL __attribute__((swiftcall))
#endif
#if !defined(SWIFT_INDIRECT_RESULT)
# define SWIFT_INDIRECT_RESULT __attribute__((swift_indirect_result))
#endif
#if !defined(SWIFT_CONTEXT)
# define SWIFT_CONTEXT __attribute__((swift_context))
#endif
#if !defined(SWIFT_ERROR_RESULT)
# define SWIFT_ERROR_RESULT __attribute__((swift_error_result))
#endif
#if defined(__cplusplus)
# define SWIFT_NOEXCEPT noexcept
#else
# define SWIFT_NOEXCEPT 
#endif
#if !defined(SWIFT_C_INLINE_THUNK)
# if __has_attribute(always_inline)
# if __has_attribute(nodebug)
#  define SWIFT_C_INLINE_THUNK inline __attribute__((always_inline)) __attribute__((nodebug))
# else
#  define SWIFT_C_INLINE_THUNK inline __attribute__((always_inline))
# endif
# else
#  define SWIFT_C_INLINE_THUNK inline
# endif
#endif
#if defined(_WIN32)
#if !defined(SWIFT_IMPORT_STDLIB_SYMBOL)
# define SWIFT_IMPORT_STDLIB_SYMBOL __declspec(dllimport)
#endif
#else
#if !defined(SWIFT_IMPORT_STDLIB_SYMBOL)
# define SWIFT_IMPORT_STDLIB_SYMBOL 
#endif
#endif
#if !__has_feature(nullability)
# define _Nonnull
# define _Nullable
# define _Null_unspecified
#elif !defined(__OBJC__)
# pragma clang diagnostic ignored "-Wnullability-extension"
#endif
#if !__has_feature(nullability_nullable_result)
# define _Nullable_result _Nullable
#endif
#if __has_feature(objc_modules)
#if __has_warning("-Watimport-in-framework-header")
#pragma clang diagnostic ignored "-Watimport-in-framework-header"
#endif
#endif

#pragma clang diagnostic ignored "-Wproperty-attribute-mismatch"
#pragma clang diagnostic ignored "-Wduplicate-method-arg"
#if __has_warning("-Wpragma-clang-attribute")
# pragma clang diagnostic ignored "-Wpragma-clang-attribute"
#endif
#pragma clang diagnostic ignored "-Wunknown-pragmas"
#pragma clang diagnostic ignored "-Wnullability"
#pragma clang diagnostic ignored "-Wdollar-in-identifier-extension"
#pragma clang diagnostic ignored "-Wunsafe-buffer-usage"

#if __has_attribute(external_source_symbol)
# pragma push_macro("any")
# undef any
# pragma clang attribute push(__attribute__((external_source_symbol(language="Swift", defined_in="MarketplaceKitWrapper",generated_declaration))), apply_to=any(function,enum,objc_interface,objc_category,objc_protocol))
# pragma pop_macro("any")
#endif

#if defined(__cplusplus)
extern "C" {
#endif

#if defined(__cplusplus)
} // extern "C"
#endif
#if __has_attribute(external_source_symbol)
# pragma clang attribute pop
#endif
#if defined(__OBJC__)
#if __has_feature(objc_modules)
#if __has_warning("-Watimport-in-framework-header")
#pragma clang diagnostic ignored "-Watimport-in-framework-header"
#endif
@import ObjectiveC;
#endif

#endif // defined(__OBJC__)
#pragma clang diagnostic ignored "-Wproperty-attribute-mismatch"
#pragma clang diagnostic ignored "-Wduplicate-method-arg"
#if __has_warning("-Wpragma-clang-attribute")
# pragma clang diagnostic ignored "-Wpragma-clang-attribute"
#endif
#pragma clang diagnostic ignored "-Wunknown-pragmas"
#pragma clang diagnostic ignored "-Wnullability"
#pragma clang diagnostic ignored "-Wdollar-in-identifier-extension"
#pragma clang diagnostic ignored "-Wunsafe-buffer-usage"

#if __has_attribute(external_source_symbol)
# pragma push_macro("any")
# undef any
# pragma clang attribute push(__attribute__((external_source_symbol(language="Swift", defined_in="MarketplaceKitWrapper",generated_declaration))), apply_to=any(function,enum,objc_interface,objc_category,objc_protocol))
# pragma pop_macro("any")
#endif

#if defined(__OBJC__)

typedef SWIFT_ENUM(NSInteger, AppDistributorType, closed) {
  AppDistributorTypeAppStore = 0,
  AppDistributorTypeTestFlight = 1,
  AppDistributorTypeMarketplace = 2,
  AppDistributorTypeWeb = 3,
  AppDistributorTypeOther = 4,
  AppDistributorTypeNotAvailable = 5,
};

@class NSString;
SWIFT_CLASS("_TtC21MarketplaceKitWrapper21AppDistributorWrapper")
@interface AppDistributorWrapper : NSObject
+ (void)getCurrentWithCompletionHandler:(void (^ _Nonnull)(enum AppDistributorType, NSString * _Nonnull))CompletionHandler;
+ (void)requestCTToken:(void (^ _Nonnull)(BOOL, NSString * _Nonnull))completionHandler;
+ (void)getEligibilityRegion:(void (^ _Nonnull)(BOOL, NSString * _Nonnull))completionHandler;
- (nonnull instancetype)init OBJC_DESIGNATED_INITIALIZER;
@end

#endif // defined(__OBJC__)
#if __has_attribute(external_source_symbol)
# pragma clang attribute pop
#endif
#if defined(__cplusplus)
#endif
#pragma clang diagnostic pop
#endif
