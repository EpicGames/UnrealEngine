// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <stdbool.h>
#include <stdint.h>

// C function declarations for the MarketplaceKitWrapper framework.
// These are weak-imported so the app can run even if the framework is unavailable.

#ifdef __cplusplus
extern "C"
{
#endif

// Swift Int maps to long on arm64 (64-bit), Swift Bool maps to C bool
typedef void (*MKW_GetCurrentCallback)(long Type, const char* Name);
typedef void (*MKW_BoolStringCallback)(bool Success, const char* Output);

extern void MKW_GetCurrent(MKW_GetCurrentCallback Callback) __attribute__((weak_import));
extern void MKW_RequestCTToken(MKW_BoolStringCallback Callback) __attribute__((weak_import));
extern void MKW_GetEligibilityRegion(MKW_BoolStringCallback Callback) __attribute__((weak_import));

#ifdef __cplusplus
}
#endif
