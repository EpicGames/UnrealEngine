// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleManager.h"

enum class EMarketplaceType : int32
{
	AppStore = 0,
	TestFlight = 1,
	Marketplace = 2,
	Web = 3,
	Other = 4,
	NotAvailable = 5,
};

const FString LexToString(EMarketplaceType Value);
void LexFromString(EMarketplaceType& OutValue, const TCHAR* InValue);

class FMarketplaceKitModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override;

	void GetCurrentTypeAsync(TFunction<void(EMarketplaceType Type, const FString& Name)> Callback);
	static void RequestCTTokenAsync(TDelegate<void(bool Success, FString Output)> Callback);
	static FString GetEligibilityRegionStatic();

	/* Returns the MarketplaceType, and BundleId */
	void GetCurrentType(EMarketplaceType& OutType, FString& OutName);
	/* Returns the MarketplaceType */
	static EMarketplaceType GetCurrentTypeStatic();
	/* Returns the BundleId of the Marketplace, when MarketplaceType == Marketplace */
	static FString GetMarketplaceBundleIdStatic();
	
	FString GetCurrentTypeAsString();
	static FString GetCurrentTypeAsStringStatic();

	/* When installed from TestFlight, GetCurrentType etc above can return an emulated value for EMarketplaceType. If you _really_ need to know if you're running from TestFlight, this will tell you. */
	bool IsTestFlight();
	static bool IsTestFlightStatic();

private:
	void GetEligibilityRegionAsync();

private:
	bool bCachedTypeValid = false;
	EMarketplaceType CachedType = EMarketplaceType::NotAvailable;
	FString CachedName;

	EMarketplaceType TestFlightMarketplaceType = EMarketplaceType::TestFlight;
	FString TestFlightMarketplaceBundleId;

	FString TestOverrideEligibilityRegion;
	FString CachedEligibilityRegion;
	TAtomic<bool> bCachedEligibilityRegionInitialized = false;

	void CacheValue();

	// Redirects TestFlight to an emulated Type/Name set by CVar for testing.
	void GetEffectiveType(EMarketplaceType& OutType, FString& OutName);
};
