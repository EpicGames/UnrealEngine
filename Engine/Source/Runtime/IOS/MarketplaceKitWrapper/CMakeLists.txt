# Copyright Epic Games, Inc. All Rights Reserved.

cmake_minimum_required(VERSION 3.26)
project(MarketplaceKitWrapper LANGUAGES Swift)

set(FRAMEWORK_NAME "MarketplaceKitWrapper")

add_library(${FRAMEWORK_NAME} SHARED MarketplaceKitWrapper.swift)
set_target_properties(${FRAMEWORK_NAME} PROPERTIES
	Swift_MODULE_NAME "${FRAMEWORK_NAME}"
	FRAMEWORK TRUE
	FRAMEWORK_VERSION A
	MACOSX_FRAMEWORK_IDENTIFIER "com.epicgames.${FRAMEWORK_NAME}"
	MACOSX_FRAMEWORK_BUNDLE_VERSION "1.0"
	MACOSX_FRAMEWORK_SHORT_VERSION_STRING "1.0"
)

# Set the install name to match the framework bundle structure
target_link_options(${FRAMEWORK_NAME} PRIVATE
	"LINKER:-install_name,@rpath/${FRAMEWORK_NAME}.framework/${FRAMEWORK_NAME}"
)

# Link required system frameworks
target_link_libraries(${FRAMEWORK_NAME} PRIVATE
	"-framework Foundation"
)

# Custom command to generate the ObjC bridging header (still needed for reference)
add_custom_command(
	OUTPUT
		${CMAKE_CURRENT_SOURCE_DIR}/MarketplaceKitWrapper.h
	DEPENDS
		${CMAKE_CURRENT_SOURCE_DIR}/MarketplaceKitWrapper.swift
	WORKING_DIRECTORY
		${CMAKE_CURRENT_SOURCE_DIR}
	COMMAND
		${CMAKE_Swift_COMPILER}
		-frontend
		-typecheck
		-target ${CMAKE_OSX_ARCHITECTURES}-apple-ios${CMAKE_OSX_DEPLOYMENT_TARGET}
		-sdk ${CMAKE_OSX_SYSROOT}
		-module-name "${FRAMEWORK_NAME}"
		-emit-objc-header-path ${CMAKE_CURRENT_SOURCE_DIR}/MarketplaceKitWrapper.h
		MarketplaceKitWrapper.swift
	COMMAND_EXPAND_LISTS
)

add_custom_target(MarketplaceKitWrapperHeader
	DEPENDS
		${CMAKE_CURRENT_SOURCE_DIR}/MarketplaceKitWrapper.h
)

add_dependencies(${FRAMEWORK_NAME} MarketplaceKitWrapperHeader)
