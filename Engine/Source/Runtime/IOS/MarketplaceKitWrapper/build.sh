#!/bin/bash
# Copyright Epic Games, Inc. All Rights Reserved.

set -e

FRAMEWORK_NAME="MarketplaceKitWrapper"

rm -rf .build
mkdir .build
pushd .build
# TODO it shouldn't be needed to specify target twice, remove CMAKE_Swift_FLAGS_INIT in future cmake versions
#XCODE_APP=${XCODE_APP:-/Applications/Xcode-26.4.app}
#export DEVELOPER_DIR="${XCODE_APP}/Contents/Developer"
IOS_SYSROOT=$(xcrun --sdk iphoneos --show-sdk-path)
cmake -G Ninja -DCMAKE_SYSTEM_NAME=iOS -DCMAKE_OSX_SYSROOT="$IOS_SYSROOT" -DCMAKE_OSX_DEPLOYMENT_TARGET=15.0 -DCMAKE_OSX_ARCHITECTURES=arm64 -DCMAKE_Swift_FLAGS_INIT="-target arm64-apple-ios15.0" ..
ninja -v
popd

# Generate the ObjC bridging header with Notice.txt prepended
cat Notice.txt MarketplaceKitWrapper.h > MarketplaceKitWrapperTemp.h
mv MarketplaceKitWrapperTemp.h MarketplaceKitWrapper.h

# Build the .framework bundle structure
FRAMEWORK_DIR="${FRAMEWORK_NAME}.framework"
rm -rf "$FRAMEWORK_DIR"
mkdir -p "$FRAMEWORK_DIR"

# Copy the dynamic library binary (CMake FRAMEWORK target puts it inside .build/<name>.framework/)
cp ".build/${FRAMEWORK_DIR}/${FRAMEWORK_NAME}" "${FRAMEWORK_DIR}/${FRAMEWORK_NAME}"

# Generate Info.plist
IOS_SDK_VERSION=$(xcrun --sdk iphoneos --show-sdk-version)
XCODE_VERSION=$(xcrun xcodebuild -version | head -1 | awk '{print $2}')
cat > "${FRAMEWORK_DIR}/Info.plist" << PLIST
<?xml version="1.0" encoding="UTF-8"?>
<!DOCTYPE plist PUBLIC "-//Apple//DTD PLIST 1.0//EN" "http://www.apple.com/DTDs/PropertyList-1.0.dtd">
<plist version="1.0">
<dict>
	<key>CFBundleDevelopmentRegion</key>
	<string>English</string>
	<key>CFBundleExecutable</key>
	<string>${FRAMEWORK_NAME}</string>
	<key>CFBundleIdentifier</key>
	<string>com.epicgames.${FRAMEWORK_NAME}</string>
	<key>CFBundleInfoDictionaryVersion</key>
	<string>6.0</string>
	<key>CFBundlePackageType</key>
	<string>FMWK</string>
	<key>CFBundleSignature</key>
	<string>????</string>
	<key>CFBundleSupportedPlatforms</key>
	<array>
		<string>iPhoneOS</string>
	</array>
	<key>CFBundleVersion</key>
	<string>1.0</string>
	<key>CFBundleShortVersionString</key>
	<string>1.0</string>
	<key>MinimumOSVersion</key>
	<string>15.0</string>
	<key>DTPlatformName</key>
	<string>iphoneos</string>
	<key>DTPlatformVersion</key>
	<string>${IOS_SDK_VERSION}</string>
	<key>DTSDKName</key>
	<string>iphoneos${IOS_SDK_VERSION}</string>
	<key>UIRequiredDeviceCapabilities</key>
	<array>
		<string>arm64</string>
	</array>
</dict>
</plist>
PLIST

# Package as .embeddedframework.zip (UBT expects this structure)
EMBEDDED_DIR="${FRAMEWORK_NAME}.embeddedframework"
rm -rf "$EMBEDDED_DIR"
mkdir -p "$EMBEDDED_DIR"
cp -R "$FRAMEWORK_DIR" "$EMBEDDED_DIR/"

ZIP_NAME="${FRAMEWORK_NAME}.embeddedframework.zip"
rm -f "$ZIP_NAME"
zip -r "$ZIP_NAME" "$EMBEDDED_DIR"

# Clean up intermediate directories
rm -rf "$EMBEDDED_DIR" "$FRAMEWORK_DIR"

# Copy zip to the MarketplaceKit module directory for UE build consumption
CONSUMER_DIR="../MarketplaceKit"
cp "$ZIP_NAME" "$CONSUMER_DIR/"

echo "Built ${ZIP_NAME} successfully"
